// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class BSPUtils : ModuleRules
    {
        public BSPUtils(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Core",
					"CoreUObject",
					"Engine"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
			);
        }
    }
}
