// Copyright Epic Games, Inc. All Rights Reserved.


#include "BSPOps.h"
#include "EngineDefines.h"
#include "Model.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Engine/BrushBuilder.h"
#include "Editor/EditorEngine.h"
#include "Components/BrushComponent.h"
#include "GameFramework/Volume.h"

DEFINE_LOG_CATEGORY_STATIC(LogBSPOps, Log, All);

/** Errors encountered in Csg operation. */
int32 FBSPOps::GErrors = 0;
bool FBSPOps::GFastRebuild = false;
FBspPointsGrid* FBspPointsGrid::GBspPoints = nullptr;
FBspPointsGrid* FBspPointsGrid::GBspVectors = nullptr;

static void TagReferencedNodes( UModel *Model, int32 *NodeRef, int32 *PolyRef, int32 iNode )
{
	FBspNode &Node = Model->Nodes[iNode];

	NodeRef[iNode     ] = 0;
	PolyRef[Node.iSurf] = 0;

	if( Node.iFront != INDEX_NONE ) TagReferencedNodes(Model,NodeRef,PolyRef,Node.iFront);
	if( Node.iBack  != INDEX_NONE ) TagReferencedNodes(Model,NodeRef,PolyRef,Node.iBack );
	if( Node.iPlane != INDEX_NONE ) TagReferencedNodes(Model,NodeRef,PolyRef,Node.iPlane);
}

//
// Update a bounding volume by expanding it to enclose a list of polys.
//
static void UpdateBoundWithPolys( FBox& Bound, FPoly** PolyList, int32 nPolys )
{
	for( int32 i=0; i<nPolys; i++ )
		for( int32 j=0; j<PolyList[i]->Vertices.Num(); j++ )
			Bound += (FVector)PolyList[i]->Vertices[j];
}

//
// Update a convolution hull with a list of polys.
//
static void UpdateConvolutionWithPolys( UModel *Model, int32 iNode, FPoly **PolyList, int32 nPolys )
{
	FBox Box(ForceInit);

	FBspNode &Node = Model->Nodes[iNode];
	Node.iCollisionBound = Model->LeafHulls.Num();
	for( int32 i=0; i<nPolys; i++ )
	{
		if( PolyList[i]->iBrushPoly != INDEX_NONE )
		{
			int32 j;
			for( j=0; j<i; j++ )
				if( PolyList[j]->iBrushPoly == PolyList[i]->iBrushPoly )
					break;
			if( j >= i )
				Model->LeafHulls.Add(PolyList[i]->iBrushPoly);
		}
		for( int32 j=0; j<PolyList[i]->Vertices.Num(); j++ )
			Box += (FVector)PolyList[i]->Vertices[j];
	}
	Model->LeafHulls.Add(INDEX_NONE);

	// Add bounds.
	Model->LeafHulls.Add( *(int32*)&Box.Min.X );
	Model->LeafHulls.Add( *(int32*)&Box.Min.Y );
	Model->LeafHulls.Add( *(int32*)&Box.Min.Z );
	Model->LeafHulls.Add( *(int32*)&Box.Max.X );
	Model->LeafHulls.Add( *(int32*)&Box.Max.Y );
	Model->LeafHulls.Add( *(int32*)&Box.Max.Z );

}

//
// Cut a partitioning poly by a list of polys, and add the resulting inside pieces to the
// front list and back list.
//
static void SplitPartitioner
(
	UModel*	Model,
	FPoly**	PolyList,
	FPoly**	FrontList,
	FPoly**	BackList,
	int32		n,
	int32		nPolys,
	int32&	nFront, 
	int32&	nBack, 
	FPoly	InfiniteEdPoly,
	TArray<FPoly*>& AllocatedFPolys
)
{
	FPoly FrontPoly,BackPoly;
	while( n < nPolys )
	{
		FPoly* Poly = PolyList[n];
		switch( InfiniteEdPoly.SplitWithPlane(Poly->Vertices[0],Poly->Normal,&FrontPoly,&BackPoly,0) )
		{
			case SP_Coplanar:
				// May occasionally happen.
//				UE_LOGF(LogBSPOps, Log,  "FilterBound: Got inficoplanar" );
				break;

			case SP_Front:
				// Shouldn't happen if hull is correct.
//				UE_LOGF(LogBSPOps, Log,  "FilterBound: Got infifront" );
				return;

			case SP_Split:
				InfiniteEdPoly = BackPoly;
				break;

			case SP_Back:
				break;
		}
		n++;
	}

	FPoly* New = new FPoly;
	*New = InfiniteEdPoly;
	New->Reverse();
	New->iBrushPoly |= 0x40000000;
	FrontList[nFront++] = New;
	AllocatedFPolys.Add( New );
	
	New = new FPoly;
	*New = InfiniteEdPoly;
	BackList[nBack++] = New;
	AllocatedFPolys.Add( New );
}

//
// Build an FPoly representing an "infinite" plane (which exceeds the maximum
// dimensions of the world in all directions) for a particular Bsp node.
//
FPoly FBSPOps::BuildInfiniteFPoly( UModel* Model, int32 iNode )
{
	FBspNode &Node   = Model->Nodes  [iNode       ];
	FBspSurf &Poly   = Model->Surfs  [Node.iSurf  ];
	FVector  Base    = FVector(Poly.Plane * Poly.Plane.W);
	FVector  Normal  = (FVector)Poly.Plane;
	FVector	 Axis1,Axis2;

	// Find two non-problematic axis vectors.
	Normal.FindBestAxisVectors( Axis1, Axis2 );

	// Set up the FPoly.
	FPoly EdPoly;
	EdPoly.Init();
	EdPoly.Normal      = (FVector3f)Normal;
	EdPoly.Base        = (FVector3f)Base;
	EdPoly.Vertices.Emplace(Base + Axis1*UE_OLD_WORLD_MAX + Axis2*UE_OLD_WORLD_MAX);
	EdPoly.Vertices.Emplace(Base - Axis1*UE_OLD_WORLD_MAX + Axis2*UE_OLD_WORLD_MAX);
	EdPoly.Vertices.Emplace(Base - Axis1*UE_OLD_WORLD_MAX - Axis2*UE_OLD_WORLD_MAX);
	EdPoly.Vertices.Emplace(Base + Axis1*UE_OLD_WORLD_MAX - Axis2*UE_OLD_WORLD_MAX);

	return EdPoly;
}

//
// Recursively filter a set of polys defining a convex hull down the Bsp,
// splitting it into two halves at each node and adding in the appropriate
// face polys at splits.
//
static void FilterBound
(
	UModel*			Model,
	FBox*			ParentBound,
	int32				iNode,
	FPoly**			PolyList,
	int32				nPolys,
	int32				Outside
)
{
	FMemMark Mark(FMemStack::Get());
	FBspNode&	Node	= Model->Nodes  [iNode];
	FBspSurf&	Surf	= Model->Surfs  [Node.iSurf];
	FVector3f	Base = Surf.Plane * Surf.Plane.W;
	FVector3f&	Normal	= Model->Vectors[Surf.vNormal];
	FBox		Bound(ForceInit);

	Bound.Min.X = Bound.Min.Y = Bound.Min.Z = +UE_OLD_WORLD_MAX;	// LWC_TODO: Should be HALF_WORLD_MAX
	Bound.Max.X = Bound.Max.Y = Bound.Max.Z = -UE_OLD_WORLD_MAX;

	// Split bound into front half and back half.
	FPoly** FrontList = new(FMemStack::Get(),nPolys*2+16)FPoly*; int32 nFront=0;
	FPoly** BackList  = new(FMemStack::Get(),nPolys*2+16)FPoly*; int32 nBack=0;

	// Keeping track of allocated FPoly structures to delete later on.
	TArray<FPoly*> AllocatedFPolys;

	FPoly* FrontPoly  = new FPoly;
	FPoly* BackPoly   = new FPoly;

	// Keep track of allocations.
	AllocatedFPolys.Add( FrontPoly );
	AllocatedFPolys.Add( BackPoly );

	for( int32 i=0; i<nPolys; i++ )
	{
		FPoly *Poly = PolyList[i];
		switch( Poly->SplitWithPlane( Base, Normal, FrontPoly, BackPoly, 0 ) )
		{
			case SP_Coplanar:
//				UE_LOGF(LogBSPOps, Log,  "FilterBound: Got coplanar" );
				FrontList[nFront++] = Poly;
				BackList[nBack++] = Poly;
				break;
			
			case SP_Front:
				FrontList[nFront++] = Poly;
				break;
			
			case SP_Back:
				BackList[nBack++] = Poly;
				break;
			
			case SP_Split:
				FrontList[nFront++] = FrontPoly;
				BackList [nBack++] = BackPoly;

				FrontPoly = new FPoly;
				BackPoly  = new FPoly;

				// Keep track of allocations.
				AllocatedFPolys.Add( FrontPoly );
				AllocatedFPolys.Add( BackPoly );

				break;

			default:
				UE_LOGF(LogBSPOps, Fatal, "FZoneFilter::FilterToLeaf: Unknown split code" );
		}
	}
	if( nFront && nBack )
	{
		// Add partitioner plane to front and back.
		FPoly InfiniteEdPoly = FBSPOps::BuildInfiniteFPoly( Model, iNode );
		InfiniteEdPoly.iBrushPoly = iNode;

		SplitPartitioner(Model,PolyList,FrontList,BackList,0,nPolys,nFront,nBack,InfiniteEdPoly,AllocatedFPolys);
	}
	else
	{
// 		if( !nFront ) UE_LOGF(LogBSPOps, Log,  "FilterBound: Empty fronthull" );
// 		if( !nBack  ) UE_LOGF(LogBSPOps, Log,  "FilterBound: Empty backhull" );
	}

	// Recursively update all our childrens' bounding volumes.
	if( nFront > 0 )
	{
		if( Node.iFront != INDEX_NONE )
			FilterBound( Model, &Bound, Node.iFront, FrontList, nFront, Outside || Node.IsCsg() );
		else if( Outside || Node.IsCsg() )
			UpdateBoundWithPolys( Bound, FrontList, nFront );
		else
			UpdateConvolutionWithPolys( Model, iNode, FrontList, nFront );
	}
	if( nBack > 0 )
	{
		if( Node.iBack != INDEX_NONE)
			FilterBound( Model, &Bound,Node.iBack, BackList, nBack, Outside && !Node.IsCsg() );
		else if( Outside && !Node.IsCsg() )
			UpdateBoundWithPolys( Bound, BackList, nBack );
		else
			UpdateConvolutionWithPolys( Model, iNode, BackList, nBack );
	}

	// Update parent bound to enclose this bound.
	if( ParentBound )
		*ParentBound += Bound;

	// Delete FPolys allocated above. We cannot use FMemStack::Get() for FPoly as the array data FPoly contains will be allocated in regular memory.
	for( int32 i=0; i<AllocatedFPolys.Num(); i++ )
	{
		FPoly* AllocatedFPoly = AllocatedFPolys[i];
		delete AllocatedFPoly;
	}

	Mark.Pop();
}

/*-----------------------------------------------------------------------------
	Bsp Splitting.
-----------------------------------------------------------------------------*/

//
// Find the best splitting polygon within a pool of polygons, and return its
// index (into the PolyList array).
//
static FPoly *FindBestSplit
(
	int32					NumPolys,
	FPoly**				PolyList,
	FBSPOps::EBspOptimization	Opt,
	int32					Balance,
	int32					InPortalBias
)
{
	check(NumPolys>0);

	// No need to test if only one poly.
	if( NumPolys==1 )
		return PolyList[0];

	FPoly   *Poly, *Best=NULL;
	float   Score, BestScore;
	int32     i, Index, j, Inc;
	int32     Splits, Front, Back, Coplanar, AllSemiSolids;

	//PortalBias -- added by Legend on 4/12/2000
	float	PortalBias = InPortalBias / 100.0f;
	Balance &= 0xFF;								// keep only the low byte to recover "Balance"
	//UE_LOGF(LogBSPOps, Log, "Balance=%d PortalBias=%f", Balance, PortalBias );

	if		(Opt==FBSPOps::BSP_Optimal)  Inc = 1;					// Test lots of nodes.
	else if (Opt==FBSPOps::BSP_Good)		Inc = FMath::Max(1,NumPolys/20);	// Test 20 nodes.
	else /* BSP_Lame */			Inc = FMath::Max(1,NumPolys/4);	// Test 4 nodes.

	// See if there are any non-semisolid polygons here.
	for( i=0; i<NumPolys; i++ )
		if( !(PolyList[i]->PolyFlags & PF_AddLast) )
			break;
	AllSemiSolids = (i>=NumPolys);

	// Search through all polygons in the pool and find:
	// A. The number of splits each poly would make.
	// B. The number of front and back nodes the polygon would create.
	// C. Number of coplanars.
	BestScore = 0;
	for( i=0; i<NumPolys; i+=Inc )
	{
		Splits = Front = Back = Coplanar = 0;
		Index = i-1;
		do
		{
			Index++;
			Poly = PolyList[Index];
		} while( Index<(i+Inc) && Index<NumPolys 
			&& ( (Poly->PolyFlags & PF_AddLast) && !(Poly->PolyFlags & PF_Portal) )
			&& !AllSemiSolids );
		if( Index>=i+Inc || Index>=NumPolys )
			continue;

		for( j=0; j<NumPolys; j+=Inc ) if( j != Index )
		{
			FPoly *OtherPoly = PolyList[j];
			switch( OtherPoly->SplitWithPlaneFast( FPlane( (FVector)Poly->Vertices[0], (FVector)Poly->Normal), NULL, NULL ) )
			{
				case SP_Coplanar:
					Coplanar++;
					break;

				case SP_Front:
					Front++;
					break;

				case SP_Back:
					Back++;
					break;

				case SP_Split:
					// Disfavor splitting polys that are zone portals.
					if( !(OtherPoly->PolyFlags & PF_Portal) )
						Splits++;
					else
						Splits += 16;
					break;
			}
		}
		// added by Legend 1/31/1999
		// Score optimization: minimize cuts vs. balance tree (as specified in BSP Rebuilder dialog)
		Score = static_cast<float>(( 100.0 - float(Balance) ) * Splits + float(Balance) * FMath::Abs( Front - Back ));
		if( Poly->PolyFlags & PF_Portal )
		{
			// PortalBias -- added by Legend on 4/12/2000
			//
			// PortalBias enables level designers to control the effect of Portals on the BSP.
			// This effect can range from 0.0 (ignore portals), to 1.0 (portals cut everything).
			//
			// In builds prior to this (since the 221 build dating back to 1/31/1999) the bias
			// has been 1.0 causing the portals to cut the BSP in ways that will potentially
			// degrade level performance, and increase the BSP complexity.
			// 
			// By setting the bias to a value between 0.3 and 0.7 the positive effects of 
			// the portals are preserved without giving them unreasonable priority in the BSP.
			//
			// Portals should be weighted high enough in the BSP to separate major parts of the
			// level from each other (pushing entire rooms down the branches of the BSP), but
			// should not be so high that portals cut through adjacent geometry in a way that
			// increases complexity of the room being (typically, accidentally) cut.
			//
			Score -= static_cast<float>(( 100.0 - float(Balance) ) * Splits * PortalBias); // ignore PortalBias of the split polys -- bias toward portal selection for cutting planes!
		}
		//UE_LOGF(LogBSPOps, Log,  "  %4d: Score = %f (Front = %4d, Back = %4d, Splits = %4d, Flags = %08X)", Index, Score, Front, Back, Splits, Poly->PolyFlags ); //LEC

		if( Score<BestScore || !Best )
		{
			Best      = Poly;
			BestScore = Score;
		}
	}
	check(Best);
	return Best;
}

//
// Pick a splitter poly then split a pool of polygons into front and back polygons and
// recurse.
//
// iParent = Parent Bsp node, or INDEX_NONE if this is the root node.
// IsFront = 1 if this is the front node of iParent, 0 of back (undefined if iParent==INDEX_NONE)
//
void FBSPOps::SplitPolyList
(
	UModel				*Model,
	int32                 iParent,
	ENodePlace			NodePlace,
	int32                 NumPolys,
	FPoly				**PolyList,
	EBspOptimization	Opt,
	int32					Balance,
	int32					PortalBias,
	int32					RebuildSimplePolys
)
{
	FMemMark Mark(FMemStack::Get());

	// Keeping track of allocated FPoly structures to delete later on.
	TArray<FPoly*> AllocatedFPolys;

	// To account for big EdPolys split up.
	int32 NumPolysToAlloc = NumPolys + 8 + NumPolys/4;
	int32 NumFront=0; FPoly **FrontList = new(FMemStack::Get(),NumPolysToAlloc)FPoly*;
	int32 NumBack =0; FPoly **BackList  = new(FMemStack::Get(),NumPolysToAlloc)FPoly*;

	FPoly *SplitPoly = FindBestSplit( NumPolys, PolyList, Opt, Balance, PortalBias );

	// Add the splitter poly to the Bsp with either a new BspSurf or an existing one.
	if( RebuildSimplePolys )
	{
		SplitPoly->iLinkSurf = Model->Surfs.Num();
	}

	int32 iOurNode	= bspAddNode(Model,iParent,NodePlace,0,SplitPoly);
	int32 iPlaneNode	= iOurNode;

	// Now divide all polygons in the pool into (A) polygons that are
	// in front of Poly, and (B) polygons that are in back of Poly.
	// Coplanar polys are inserted immediately, before recursing.

	// If any polygons are split by Poly, we ignrore the original poly,
	// split it into two polys, and add two new polys to the pool.
	FPoly *FrontEdPoly = new FPoly;
	FPoly *BackEdPoly  = new FPoly;
	// Keep track of allocations.
	AllocatedFPolys.Add( FrontEdPoly );
	AllocatedFPolys.Add( BackEdPoly );

	for( int32 i=0; i<NumPolys; i++ )
	{
		FPoly *EdPoly = PolyList[i];
		if( EdPoly == SplitPoly )
		{
			continue;
		}

		switch( EdPoly->SplitWithPlane( SplitPoly->Vertices[0], SplitPoly->Normal, FrontEdPoly, BackEdPoly, 0 ) )
		{
			case SP_Coplanar:
	            if( RebuildSimplePolys )
				{
					EdPoly->iLinkSurf = Model->Surfs.Num()-1;
				}
				iPlaneNode = bspAddNode( Model, iPlaneNode, NODE_Plane, 0, EdPoly );
				break;
			
			case SP_Front:
	            FrontList[NumFront++] = PolyList[i];
				break;
			
			case SP_Back:
	            BackList[NumBack++] = PolyList[i];
				break;
			
			case SP_Split:

				// Create front & back nodes.
				FrontList[NumFront++] = FrontEdPoly;
				BackList [NumBack ++] = BackEdPoly;

				FrontEdPoly = new FPoly;
				BackEdPoly  = new FPoly;
				// Keep track of allocations.
				AllocatedFPolys.Add( FrontEdPoly );
				AllocatedFPolys.Add( BackEdPoly );

				break;
		}
	}

	// Recursively split the front and back pools.
	if( NumFront > 0 ) SplitPolyList( Model, iOurNode, NODE_Front, NumFront, FrontList, Opt, Balance, PortalBias, RebuildSimplePolys );
	if( NumBack  > 0 ) SplitPolyList( Model, iOurNode, NODE_Back,  NumBack,  BackList,  Opt, Balance, PortalBias, RebuildSimplePolys );

	// Delete FPolys allocated above. We cannot use FMemStack::Get() for FPoly as the array data FPoly contains will be allocated in regular memory.
	for( int32 i=0; i<AllocatedFPolys.Num(); i++ )
	{
		FPoly* AllocatedFPoly = AllocatedFPolys[i];
		delete AllocatedFPoly;
	}

	Mark.Pop();
}

/** Prepare a moving brush. */
void FBSPOps::csgPrepMovingBrush( ABrush* Actor )
{
	check(Actor);
//	UE_LOGF(LogBSPOps, Log,  "Preparing brush %ls", *Actor->GetName() );  // moved here so that we can easily debug when an actor has lost parts of its brush

	check(Actor->GetBrushComponent());
	check(Actor->Brush);
	check(Actor->Brush->RootOutside);

	RebuildBrush(Actor->Brush);

	// Make sure simplified collision is up to date.
	Actor->GetBrushComponent()->BuildSimpleBrushCollision();
	Actor->RebuildNavigationData();
}

/**
 * Duplicates the specified brush and makes it into a CSG-able level brush.
 * @return		The new brush, or NULL if the original was empty.
 */
void FBSPOps::csgCopyBrush( ABrush* Dest, ABrush* Src, uint32 PolyFlags, EObjectFlags ResFlags, bool bNeedsPrep, bool bCopyPosRotScale, bool bAllowEmpty )
{
	check(Src);
	check(Src->GetBrushComponent());
	check(Src->Brush);

	// Handle empty brush.
	if( !bAllowEmpty && !Src->Brush->Polys->Element.Num() )
	{
		Dest->Brush = NULL;
		Dest->GetBrushComponent()->Brush = NULL;
		return;
	}

	// Duplicate the brush and its polys.
	Dest->PolyFlags		= PolyFlags;
	Dest->Brush = NewObject<UModel>(Dest, NAME_None, ResFlags);
	Dest->Brush->Initialize(nullptr, Src->Brush->RootOutside);
	Dest->Brush->Polys	= NewObject<UPolys>(Dest->Brush, NAME_None, ResFlags);
	Dest->Brush->Polys->Element = Src->Brush->Polys->Element;
	Dest->GetBrushComponent()->Brush = Dest->Brush;
	if(Src->BrushBuilder != nullptr)
	{
		Dest->BrushBuilder = DuplicateObject<UBrushBuilder>(Src->BrushBuilder, Dest);
	}

	// Update poly textures.
	for( int32 i=0; i<Dest->Brush->Polys->Element.Num(); i++ )
	{
		Dest->Brush->Polys->Element[i].iBrushPoly = INDEX_NONE;
	}

	// Copy positioning, and build bounding box.
	if(bCopyPosRotScale)
	{
		Dest->CopyPosRotScaleFrom( Src );
	}

	// If it's a moving brush, prep it.
	if( bNeedsPrep )
	{
		csgPrepMovingBrush( Dest );
	}
}

/**
 * Adds a brush to the list of CSG brushes in the level, using a CSG operation.
 *
 * @return		A newly-created copy of the brush.
 */
ABrush*	FBSPOps::csgAddOperation( ABrush* Actor, uint32 PolyFlags, EBrushType BrushType)
{
	check(Actor);
	check(Actor->GetBrushComponent());
	check(Actor->Brush);
	check(Actor->Brush->Polys);
	check(Actor->GetWorld());

	// Can't do this if brush has no polys. 
	if( !Actor->Brush->Polys->Element.Num() )
		return NULL;

	// Spawn a new actor for the brush.

	ABrush* Result  = Actor->GetWorld()->SpawnBrush();
	Result->SetNotForClientOrServer();

	// Duplicate the brush.
	csgCopyBrush
	(
		Result,
		Actor,
		PolyFlags,
		RF_Transactional,
		0,
		true
	);
	check(Result->Brush);

	// Assign the default material to the brush's polys.
	for( int32 i=0; i<Result->Brush->Polys->Element.Num(); i++ )
	{
		FPoly& CurrentPoly = Result->Brush->Polys->Element[i];
		if ( !CurrentPoly.Material )
		{
			CurrentPoly.Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	// Set add-info.
	Result->BrushType = BrushType;

	Result->ReregisterAllComponents();

	return Result;
}

/** Add a new point to the model (preventing duplicates) and return its index. */
static int32 AddThing( TArray<FVector3f>& Vectors, const FVector3f& V, float Thresh, int32 Check )
{
	if( Check )
	{
		// See if this is very close to an existing point/vector.		
		for( int32 i=0; i<Vectors.Num(); i++ )
		{
			const FVector3f &TableVect = Vectors[i];
			float Temp=(V.X - TableVect.X);
			if( (Temp > -Thresh) && (Temp < Thresh) )
			{
				Temp=(V.Y - TableVect.Y);
				if( (Temp > -Thresh) && (Temp < Thresh) )
				{
					Temp=(V.Z - TableVect.Z);
					if( (Temp > -Thresh) && (Temp < Thresh) )
					{
						// Found nearly-matching vector.
						return i;
					}
				}
			}
		}
	}
	return Vectors.Add( V );
}

/** Add a new vector to the model, merging near-duplicates, and return its index. */
int32 FBSPOps::bspAddVector( UModel* Model, const FVector* V, bool Exact )
{
	const float Thresh = Exact ? THRESH_NORMALS_ARE_SAME : THRESH_VECTORS_ARE_NEAR;

	if (FBspPointsGrid::GBspVectors)
	{
		// If a points grid has been built for quick vector lookup, use that instead of doing a linear search
		const int32 NextIndex = Model->Vectors.Num();
		const int32 ReturnedIndex = FBspPointsGrid::GBspVectors->FindOrAddPoint(*V, NextIndex, Thresh);
		if (ReturnedIndex == NextIndex)
		{
			Model->Vectors.Add((FVector3f)*V);
		}

		return ReturnedIndex;
	}

	return AddThing
	(
		Model->Vectors,
		(FVector3f)*V,
		Exact ? THRESH_NORMALS_ARE_SAME : THRESH_VECTORS_ARE_NEAR,
		1
	);
}

/** Add a new point to the model, merging near-duplicates, and return its index. */
int32 FBSPOps::bspAddPoint( UModel* Model, const FVector* V, bool Exact )
{
	const float Thresh = Exact ? THRESH_POINTS_ARE_SAME : THRESH_POINTS_ARE_NEAR;

	if (FBspPointsGrid::GBspPoints)
	{
		// If a points grid has been built for quick point lookup, use that instead of doing a linear search
		const int32 NextIndex = Model->Points.Num();
		// Always look for points with a low threshold; a generous threshold can result in 'leaks' in the BSP and unwanted polys being generated
		const int32 ReturnedIndex = FBspPointsGrid::GBspPoints->FindOrAddPoint(*V, NextIndex, THRESH_POINTS_ARE_SAME);
		if (ReturnedIndex == NextIndex)
		{
			Model->Points.Add((FVector3f)*V);
		}

		return ReturnedIndex;
	}

	// Try to find a match quickly from the Bsp. This finds all potential matches
	// except for any dissociated from nodes/surfaces during a rebuild.
	FVector3f Temp;
	int32 pVertex;
	float NearestDist = Model->FindNearestVertex((FVector3f)*V,Temp,Thresh,pVertex);
	if( (NearestDist >= 0.0) && (NearestDist <= Thresh) )
	{
		// Found an existing point.
		return pVertex;
	}
	else
	{
		// No match found; add it slowly to find duplicates.
		return AddThing(Model->Points, (FVector3f)*V, Thresh, !GFastRebuild);
	}
}


/**
 * Builds Bsp from the editor polygon set (EdPolys) of a model.
 *
 * Opt     = Bsp optimization, BSP_Lame (fast), BSP_Good (medium), BSP_Optimal (slow)
 * Balance = 0-100, 0=only worry about minimizing splits, 100=only balance tree.
 */
void FBSPOps::bspBuild( UModel* Model, enum EBspOptimization Opt, int32 Balance, int32 PortalBias, int32 RebuildSimplePolys, int32 iNode )
{
	int32 OriginalPolys = Model->Polys->Element.Num();

	// Empty the model's tables.
	if( RebuildSimplePolys==1 )
	{
		// Empty everything but polys.
		Model->EmptyModel( 1, 0 );
	}
	else if( RebuildSimplePolys==0 )
	{
		// Empty node vertices.
		for( int32 i=0; i<Model->Nodes.Num(); i++ )
			Model->Nodes[i].NumVertices = 0;

		// Refresh the Bsp.
		bspRefresh(Model,1);
		
		// Empty nodes.
		Model->EmptyModel( 0, 0 );
	}
	if( Model->Polys->Element.Num() )
	{
		// Allocate polygon pool.
		FMemMark Mark(FMemStack::Get());
		FPoly** PolyList = new( FMemStack::Get(), Model->Polys->Element.Num() )FPoly*;

		// Add all FPolys to active list.
		for( int32 i=0; i<Model->Polys->Element.Num(); i++ )
			if( Model->Polys->Element[i].Vertices.Num() )
				PolyList[i] = &Model->Polys->Element[i];

		// Now split the entire Bsp by splitting the list of all polygons.
		SplitPolyList
		(
			Model,
			INDEX_NONE,
			NODE_Root,
			Model->Polys->Element.Num(),
			PolyList,
			Opt,
			Balance,
			PortalBias,
			RebuildSimplePolys
		);

		// Now build the bounding boxes for all nodes.
		if( RebuildSimplePolys==0 )
		{
			// Remove unreferenced things.
			bspRefresh( Model, 1 );

			// Rebuild all bounding boxes.
			bspBuildBounds( Model );
		}

		Mark.Pop();
	}

//	UE_LOGF(LogBSPOps, Log,  "bspBuild built %i convex polys into %i nodes", OriginalPolys, Model->Nodes.Num() );
}

/**
 * If the Bsp's point and vector tables are nearly full, reorder them and delete unused ones.
 */
void FBSPOps::bspRefresh( UModel* Model, bool NoRemapSurfs )
{
	FMemStack& MemStack = FMemStack::Get();

	FMemMark Mark(MemStack);

	int32 NumNodes = Model->Nodes.Num();
	int32 NumSurfs = Model->Surfs.Num();
	int32 NumVectors = Model->Vectors.Num();
	int32 NumPoints = Model->Points.Num();

	// Remove unreferenced Bsp surfs.
	int32* PolyRef;
	if( NoRemapSurfs )
	{
		PolyRef = NewZeroed<int32>(MemStack, NumSurfs);
	}
	else
	{
		PolyRef = NewOned<int32>(MemStack, NumSurfs);
	}

	int32* NodeRef = NewOned<int32>(MemStack, NumNodes);
	if( NumNodes > 0 )
	{
		TagReferencedNodes( Model, NodeRef, PolyRef, 0 );
	}

	// Remap Bsp surfs.
	{
		int32 n=0;
		for( int32 i=0; i<NumSurfs; i++ )
		{
			if( PolyRef[i]!=INDEX_NONE )
			{
				Model->Surfs[n] = Model->Surfs[i];
				PolyRef[i]=n++;
			}
		}
		//UE_LOGF(LogBSPOps, Log,  "Polys: %i -> %i", NumSurfs, n );
		Model->Surfs.RemoveAt( n, NumSurfs-n );
		NumSurfs = n;
	}

	// Remap Bsp nodes.
	{
		int32 n=0;
		for( int32 i=0; i<NumNodes; i++ )
		{
			if( NodeRef[i]!=INDEX_NONE )
			{
				Model->Nodes[n] = Model->Nodes[i];
				NodeRef[i]=n++;
			}
		}
		//UE_LOGF(LogBSPOps, Log,  "Nodes: %i -> %i", NumNodes, n );
		Model->Nodes.RemoveAt( n, NumNodes-n );
		NumNodes = n;
	}

	// Update Bsp nodes.
	for( int32 i=0; i<NumNodes; i++ )
	{
		FBspNode *Node = &Model->Nodes[i];
		Node->iSurf = PolyRef[Node->iSurf];
		if (Node->iFront != INDEX_NONE) Node->iFront = NodeRef[Node->iFront];
		if (Node->iBack  != INDEX_NONE) Node->iBack  = NodeRef[Node->iBack];
		if (Node->iPlane != INDEX_NONE) Node->iPlane = NodeRef[Node->iPlane];
	}

	// Remove unreferenced points and vectors.
	int32* VectorRef = NewOned<int32>(MemStack, NumVectors);
	int32* PointRef  = NewOned<int32>(MemStack, NumPoints);

	// Check Bsp surfs.
	TArray<int32*> VertexRef;
	for( int32 i=0; i<NumSurfs; i++ )
	{
		FBspSurf *Surf = &Model->Surfs[i];
		VectorRef [Surf->vNormal   ] = 0;
		VectorRef [Surf->vTextureU ] = 0;
		VectorRef [Surf->vTextureV ] = 0;
		PointRef  [Surf->pBase     ] = 0;
	}

	// Check Bsp nodes.
	for( int32 i=0; i<NumNodes; i++ )
	{
		// Tag all points used by nodes.
		FBspNode*	Node		= &Model->Nodes[i];
		FVert*		VertPool	= &Model->Verts[Node->iVertPool];
		for( int B=0; B<Node->NumVertices;  B++ )
		{
			PointRef[VertPool->pVertex] = 0;
			VertPool++;
		}
	}

	// Remap points.
	{
		int32 n=0;
		for( int32 i=0; i<NumPoints; i++ ) if( PointRef[i]!=INDEX_NONE )
		{
			Model->Points[n] = Model->Points[i];
			PointRef[i] = n++;
		}
		//UE_LOGF(LogBSPOps, Log,  "Points: %i -> %i", NumPoints, n );
		Model->Points.RemoveAt( n, NumPoints-n );
		NumPoints = n;
	}

	// Remap vectors.
	{
		int32 n=0;
		for (int32 i=0; i<NumVectors; i++) if (VectorRef[i]!=INDEX_NONE)
		{
			Model->Vectors[n] = Model->Vectors[i];
			VectorRef[i] = n++;
		}
		//UE_LOGF(LogBSPOps, Log,  "Vectors: %i -> %i", NumVectors, n );
		Model->Vectors.RemoveAt( n, NumVectors-n );
		NumVectors = n;
	}

	// Update Bsp surfs.
	for( int32 i=0; i<NumSurfs; i++ )
	{
		FBspSurf *Surf	= &Model->Surfs[i];
		Surf->vNormal   = VectorRef [Surf->vNormal  ];
		Surf->vTextureU = VectorRef [Surf->vTextureU];
		Surf->vTextureV = VectorRef [Surf->vTextureV];
		Surf->pBase     = PointRef  [Surf->pBase    ];
	}

	// Update Bsp nodes.
	for( int32 i=0; i<NumNodes; i++ )
	{
		FBspNode*	Node		= &Model->Nodes[i];
		FVert*		VertPool	= &Model->Verts[Node->iVertPool];
		for( int B=0; B<Node->NumVertices;  B++ )
		{			
			VertPool->pVertex = PointRef [VertPool->pVertex];
			VertPool++;
		}
	}

	// Shrink the objects.
	Model->ShrinkModel();

	Mark.Pop();
}

// Build bounding volumes for all Bsp nodes.  The bounding volume of the node
// completely encloses the "outside" space occupied by the nodes.  Note that 
// this is not the same as representing the bounding volume of all of the 
// polygons within the node.
//
// We start with a practically-infinite cube and filter it down the Bsp,
// whittling it away until all of its convex volume fragments land in leaves.
void FBSPOps::bspBuildBounds( UModel* Model )
{
	if( Model->Nodes.Num()==0 )
		return;

	FPoly Polys[6], *PolyList[6];
	for( int32 i=0; i<6; i++ )
	{
		PolyList[i] = &Polys[i];
		PolyList[i]->Init();
		PolyList[i]->iBrushPoly = INDEX_NONE;
	}

	Polys[0].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX,HALF_WORLD_MAX);
	Polys[0].Vertices.Emplace( HALF_WORLD_MAX,-HALF_WORLD_MAX,HALF_WORLD_MAX);
	Polys[0].Vertices.Emplace( HALF_WORLD_MAX, HALF_WORLD_MAX,HALF_WORLD_MAX);
	Polys[0].Vertices.Emplace(-HALF_WORLD_MAX, HALF_WORLD_MAX,HALF_WORLD_MAX);
	Polys[0].Normal   =FVector3f( 0.000000,  0.000000,  1.000000 );
	Polys[0].Base     =Polys[0].Vertices[0];

	Polys[1].Vertices.Emplace(-HALF_WORLD_MAX, HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[1].Vertices.Emplace( HALF_WORLD_MAX, HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[1].Vertices.Emplace( HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[1].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[1].Normal   =FVector3f( 0.000000,  0.000000, -1.000000 );
	Polys[1].Base     =Polys[1].Vertices[0];

	Polys[2].Vertices.Emplace(-HALF_WORLD_MAX,HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[2].Vertices.Emplace(-HALF_WORLD_MAX,HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[2].Vertices.Emplace( HALF_WORLD_MAX,HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[2].Vertices.Emplace( HALF_WORLD_MAX,HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[2].Normal   =FVector3f( 0.000000,  1.000000,  0.000000 );
	Polys[2].Base     =Polys[2].Vertices[0];

	Polys[3].Vertices.Emplace( HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[3].Vertices.Emplace( HALF_WORLD_MAX,-HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[3].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[3].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[3].Normal   =FVector3f( 0.000000, -1.000000,  0.000000 );
	Polys[3].Base     =Polys[3].Vertices[0];

	Polys[4].Vertices.Emplace(HALF_WORLD_MAX, HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[4].Vertices.Emplace(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[4].Vertices.Emplace(HALF_WORLD_MAX,-HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[4].Vertices.Emplace(HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[4].Normal   =FVector3f( 1.000000,  0.000000,  0.000000 );
	Polys[4].Base     =Polys[4].Vertices[0];

	Polys[5].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[5].Vertices.Emplace(-HALF_WORLD_MAX,-HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[5].Vertices.Emplace(-HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX);
	Polys[5].Vertices.Emplace(-HALF_WORLD_MAX, HALF_WORLD_MAX,-HALF_WORLD_MAX);
	Polys[5].Normal   =FVector3f(-1.000000,  0.000000,  0.000000 );
	Polys[5].Base     =Polys[5].Vertices[0];
	// Empty hulls.
	Model->LeafHulls.Empty();
	for( int32 i=0; i<Model->Nodes.Num(); i++ )
		Model->Nodes[i].iCollisionBound  = INDEX_NONE;
	FilterBound( Model, NULL, 0, PolyList, 6, Model->RootOutside );
//	UE_LOGF(LogBSPOps, Log,  "bspBuildBounds: Generated %i hulls", Model->LeafHulls.Num() );
}

/**
 * Validate a brush, and set iLinks on all EdPolys to index of the
 * first identical EdPoly in the list, or its index if it's the first.
 * Not transactional.
 */
void FBSPOps::bspValidateBrush( UModel* Brush, bool ForceValidate, bool DoStatusUpdate )
{
	check(Brush != nullptr);
	Brush->Modify(false);
	if( ForceValidate || !Brush->Linked )
	{
		Brush->Linked = 1;
		for( int32 i=0; i<Brush->Polys->Element.Num(); i++ )
		{
			Brush->Polys->Element[i].iLink = i;
		}
		int32 n=0;
		for( int32 i=0; i<Brush->Polys->Element.Num(); i++ )
		{
			FPoly* EdPoly = &Brush->Polys->Element[i];
			if( EdPoly->iLink==i )
			{
				for( int32 j=i+1; j<Brush->Polys->Element.Num(); j++ )
				{
					FPoly* OtherPoly = &Brush->Polys->Element[j];
					if
					(	OtherPoly->iLink == j
					&&	OtherPoly->Material == EdPoly->Material
					&&	OtherPoly->TextureU == EdPoly->TextureU
					&&	OtherPoly->TextureV == EdPoly->TextureV
					&&	OtherPoly->PolyFlags == EdPoly->PolyFlags
					&&	(OtherPoly->Normal | EdPoly->Normal)>0.9999 )
					{
						float Dist = FVector3f::PointPlaneDist( OtherPoly->Vertices[0], EdPoly->Vertices[0], EdPoly->Normal );
						if( Dist>-0.001 && Dist<0.001 )
						{
							OtherPoly->iLink = i;
							n++;
						}
					}
				}
			}
		}
// 		UE_LOGF(LogBSPOps, Log,  "BspValidateBrush linked %i of %i polys", n, Brush->Polys->Element.Num() );
	}

	// Build bounds.
	Brush->BuildBound();
}

void FBSPOps::bspUnlinkPolys( UModel* Brush )
{
	Brush->Modify(false);
	Brush->Linked = 1;
	for( int32 i=0; i<Brush->Polys->Element.Num(); i++ )
	{
		Brush->Polys->Element[i].iLink = i;
	}
}

// Add an editor polygon to the Bsp, and also stick a reference to it
// in the editor polygon's BspNodes list. If the editor polygon has more sides
// than the Bsp will allow, split it up into several sub-polygons.
//
// Returns: Index to newly-created node of Bsp.  If several nodes were created because
// of split polys, returns the parent (highest one up in the Bsp).
int32	FBSPOps::bspAddNode( UModel* Model, int32 iParent, ENodePlace NodePlace, uint32 NodeFlags, FPoly* EdPoly )
{
	if( NodePlace == NODE_Plane )
	{
		// Make sure coplanars are added at the end of the coplanar list so that 
		// we don't insert NF_IsNew nodes with non NF_IsNew coplanar children.
		while( Model->Nodes[iParent].iPlane != INDEX_NONE )
		{
			iParent = Model->Nodes[iParent].iPlane;
		}
	}
	FBspSurf* Surf = NULL;
	if( EdPoly->iLinkSurf == Model->Surfs.Num() )
	{
		int32 NewIndex = Model->Surfs.AddZeroed();
		Surf = &Model->Surfs[NewIndex];

		// This node has a new polygon being added by bspBrushCSG; must set its properties here.
		FVector Base = (FVector)EdPoly->Base;
		FVector Normal = (FVector)EdPoly->Normal;
		FVector TextureU = (FVector)EdPoly->TextureU;
		FVector TextureV = (FVector)EdPoly->TextureV;

		Surf->pBase     	= bspAddPoint  (Model,&Base,1);
		Surf->vNormal   	= bspAddVector (Model,&Normal,1);
		Surf->vTextureU 	= bspAddVector (Model,&TextureU,0);
		Surf->vTextureV 	= bspAddVector (Model,&TextureV,0);
		Surf->Material  	= EdPoly->Material;
		Surf->Actor			= NULL;

		Surf->PolyFlags 	= EdPoly->PolyFlags & ~PF_NoAddToBSP;
		Surf->LightMapScale= EdPoly->LightMapScale;

		// Find the LightmassPrimitiveSettings in the UModel...
		int32 FoundLightmassIndex = INDEX_NONE;
		if (Model->LightmassSettings.Find(EdPoly->LightmassSettings, FoundLightmassIndex) == false)
		{
			FoundLightmassIndex = Model->LightmassSettings.Add(EdPoly->LightmassSettings);
		}
		Surf->iLightmassIndex = FoundLightmassIndex;

		Surf->Actor	 		= EdPoly->Actor;
		Surf->iBrushPoly	= EdPoly->iBrushPoly;
		
		if (EdPoly->Actor)
		{
			Surf->bHiddenEdTemporary = EdPoly->Actor->IsTemporarilyHiddenInEditor();
			Surf->bHiddenEdLevel = EdPoly->Actor->bHiddenEdLevel;
			Surf->bHiddenEdLayer = EdPoly->Actor->bHiddenEdLayer;
		}

		Surf->Plane			= FPlane4f(EdPoly->Vertices[0],EdPoly->Normal);
	}
	else
	{
		check(EdPoly->iLinkSurf!=INDEX_NONE);
		check(EdPoly->iLinkSurf<Model->Surfs.Num());
		Surf = &Model->Surfs[EdPoly->iLinkSurf];
	}

	// Set NodeFlags.
	if( Surf->PolyFlags & PF_NotSolid              ) NodeFlags |= NF_NotCsg;
	if( Surf->PolyFlags & (PF_Invisible|PF_Portal) ) NodeFlags |= NF_NotVisBlocking;

	if( EdPoly->Vertices.Num() > FBspNode::MAX_NODE_VERTICES )
	{
		// Split up into two coplanar sub-polygons (one with MAX_NODE_VERTICES vertices and
		// one with all the remaining vertices) and recursively add them.

		// EdPoly1 is just the first MAX_NODE_VERTICES from EdPoly.
		FMemMark Mark(FMemStack::Get());
		FPoly *EdPoly1 = new FPoly;
		*EdPoly1 = *EdPoly;
		EdPoly1->Vertices.RemoveAt(FBspNode::MAX_NODE_VERTICES,EdPoly->Vertices.Num() - FBspNode::MAX_NODE_VERTICES);

		// EdPoly2 is the first vertex from EdPoly, and the last EdPoly->Vertices.Num() - MAX_NODE_VERTICES + 1.
		FPoly *EdPoly2 = new FPoly;
		*EdPoly2 = *EdPoly;
		EdPoly2->Vertices.RemoveAt(1,FBspNode::MAX_NODE_VERTICES - 2);

		int32 iNode = bspAddNode( Model, iParent, NodePlace, NodeFlags, EdPoly1 ); // Add this poly first.
		bspAddNode( Model, iNode,   NODE_Plane, NodeFlags, EdPoly2 ); // Then add other (may be bigger).

		delete EdPoly1;
		delete EdPoly2;

		Mark.Pop();
		return iNode; // Return coplanar "parent" node (not coplanar child)
	}
	else
	{
		// Add node.
		int32 iNode			 = Model->Nodes.AddZeroed();
		FBspNode& Node       = Model->Nodes[iNode];

		// Tell transaction tracking system that parent is about to be modified.
		FBspNode* Parent=NULL;
		if( NodePlace!=NODE_Root )
			Parent = &Model->Nodes[iParent];

		// Set node properties.
		Node.iSurf       	 = EdPoly->iLinkSurf;
		Node.NodeFlags   	 = static_cast<uint8>(NodeFlags);
		Node.iCollisionBound = INDEX_NONE;
		Node.Plane           = FPlane4f( EdPoly->Vertices[0], EdPoly->Normal );
		Node.iVertPool       = Model->Verts.AddUninitialized(EdPoly->Vertices.Num());
		Node.iFront		     = INDEX_NONE;
		Node.iBack		     = INDEX_NONE;
		Node.iPlane		     = INDEX_NONE;
		if( NodePlace==NODE_Root )
		{
			Node.iLeaf[0]	 = INDEX_NONE;
			Node.iLeaf[1] 	 = INDEX_NONE;
			Node.iZone[0]	 = 0;
			Node.iZone[1]	 = 0;
		}
		else if( NodePlace==NODE_Front || NodePlace==NODE_Back )
		{
			int32 ZoneFront=NodePlace==NODE_Front;
			Node.iLeaf[0]	 = Parent->iLeaf[ZoneFront];
			Node.iLeaf[1] 	 = Parent->iLeaf[ZoneFront];
			Node.iZone[0]	 = Parent->iZone[ZoneFront];
			Node.iZone[1]	 = Parent->iZone[ZoneFront];
		}
		else
		{
			int32 IsFlipped    = (Node.Plane|Parent->Plane)<0.0;
			Node.iLeaf[0]    = Parent->iLeaf[IsFlipped  ];
			Node.iLeaf[1]    = Parent->iLeaf[1-IsFlipped];
			Node.iZone[0]    = Parent->iZone[IsFlipped  ];
			Node.iZone[1]    = Parent->iZone[1-IsFlipped];
		}

		// Link parent to this node.
		if (NodePlace == NODE_Front)
		{
			Parent->iFront = iNode;
		}
		else if (NodePlace == NODE_Back)
		{
			Parent->iBack = iNode;
		}
		else if (NodePlace == NODE_Plane)
		{
			Parent->iPlane = iNode;
		}

		// Add all points to point table, merging nearly-overlapping polygon points
		// with other points in the poly to prevent criscrossing vertices from
		// being generated.

		// Must maintain Node->NumVertices on the fly so that bspAddPoint is always
		// called with the Bsp in a clean state.
		Node.NumVertices = 0;
		FVert* VertPool	 = &Model->Verts[ Node.iVertPool ];
		for( uint8 i=0; i<EdPoly->Vertices.Num(); i++ )
		{
			FVector Vertex = (FVector)EdPoly->Vertices[i];
			int32 pVertex = bspAddPoint(Model,&Vertex,0);
			if( Node.NumVertices==0 || VertPool[Node.NumVertices-1].pVertex!=pVertex )
			{
				VertPool[Node.NumVertices].iSide   = INDEX_NONE;
				VertPool[Node.NumVertices].pVertex = pVertex;
				Node.NumVertices++;
			}
		}
		if( Node.NumVertices>=2 && VertPool[0].pVertex==VertPool[Node.NumVertices-1].pVertex )
		{
			Node.NumVertices--;
		}
		if( Node.NumVertices < 3 )
		{
			GErrors++;
// 			UE_LOGF(LogBSPOps, Warning, "bspAddNode: Infinitesimal polygon %i (%i)", Node.NumVertices, EdPoly->Vertices.Num() );
			Node.NumVertices = 0;
		}

		return iNode;
	}
}

/**
 * Rebuild some brush internals
 */
void FBSPOps::RebuildBrush(UModel* Brush)
{
	Brush->Modify(false);
	Brush->EmptyModel(1, 0);

	// Build bounding box.
	Brush->BuildBound();

	// Build BSP for the brush.
	bspBuild(Brush, BSP_Good, 15, 70, 1, 0);
	bspRefresh(Brush, 1);
	bspBuildBounds(Brush);
}

/**
 * Rotates the specified brush's vertices.
 */
void FBSPOps::RotateBrushVerts(ABrush* Brush, const FRotator& Rotation, bool bClearComponents)
{
	if(Brush->GetBrushComponent()->Brush && Brush->GetBrushComponent()->Brush->Polys)
	{
		for( int32 poly = 0 ; poly < Brush->GetBrushComponent()->Brush->Polys->Element.Num() ; poly++ )
		{
			FPoly* Poly = &(Brush->GetBrushComponent()->Brush->Polys->Element[poly]);

			// Rotate the vertices.
			const FRotationMatrix RotMatrix( Rotation );
			for( int32 vertex = 0 ; vertex < Poly->Vertices.Num() ; vertex++ )
			{
				Poly->Vertices[vertex] = FVector3f(Brush->GetPivotOffset() + (FVector)RotMatrix.TransformVector((FVector)Poly->Vertices[vertex] - Brush->GetPivotOffset()));
			}
			Poly->Base = FVector3f(Brush->GetPivotOffset() + (FVector)RotMatrix.TransformVector(FVector(Poly->Base) - Brush->GetPivotOffset()));

			// Rotate the texture vectors.
			Poly->TextureU = FVector4f(RotMatrix.TransformVector( (FVector)Poly->TextureU ));
			Poly->TextureV = FVector4f(RotMatrix.TransformVector( (FVector)Poly->TextureV ));

			// Recalc the normal for the poly.
			Poly->Normal = FVector3f::ZeroVector;
			Poly->Finalize(Brush,0);
		}

		Brush->GetBrushComponent()->Brush->BuildBound();

		if( !Brush->IsStaticBrush() )
		{
			csgPrepMovingBrush( Brush );
		}

		if ( bClearComponents )
		{
			Brush->ReregisterAllComponents();
		}
	}
}


void FBSPOps::HandleVolumeShapeChanged(AVolume& Volume)
{
	// The default physics volume doesn't have an associated UModel, so we need to handle that case gracefully.
	if(Volume.Brush)
	{
		FBSPOps::csgPrepMovingBrush( &Volume );
	}
}


void FBspPointsGrid::Clear(int32 InitialSize)
{
	GridMap.Empty(InitialSize);
}


// Given a grid index in one axis, a real position on the grid and a threshold radius,
// return either:
// - the additional grid index it can overlap in that axis, or
// - the original grid index if there is no overlap.
static FORCEINLINE int32 GetAdjacentIndexIfOverlapping(int32 GridIndex, float GridPos, float GridThreshold)
{
	if (GridPos - GridIndex < GridThreshold)
	{
		return GridIndex - 1;
	}
	else if (1.0f - (GridPos - GridIndex) < GridThreshold)
	{
		return GridIndex + 1;
	}
	else
	{
		return GridIndex;
	}
}

int32 FBspPointsGrid::FindOrAddPoint(const FVector& Point, int32 Index, float PointThreshold)
{
	// Offset applied to the grid coordinates so aligned vertices (the normal case) don't overlap several grid items (taking into account the threshold)
	const float GridOffset = 0.12345f;

	const float AdjustedPointX = static_cast<float>(Point.X - GridOffset);
	const float AdjustedPointY = static_cast<float>(Point.Y - GridOffset);
	const float AdjustedPointZ = static_cast<float>(Point.Z - GridOffset);

	const float GridX = AdjustedPointX * OneOverGranularity;
	const float GridY = AdjustedPointY * OneOverGranularity;
	const float GridZ = AdjustedPointZ * OneOverGranularity;

	// Get the grid indices corresponding to the point coordinates
	const int32 GridIndexX = FMath::FloorToInt(GridX);
	const int32 GridIndexY = FMath::FloorToInt(GridY);
	const int32 GridIndexZ = FMath::FloorToInt(GridZ);

	// Find grid item in map
	FBspPointsGridItem& GridItem = GridMap.FindOrAdd(FBspPointsKey(GridIndexX, GridIndexY, GridIndexZ));

	// Iterate through grid item points and return a point if it's close to the threshold
	const float PointThresholdSquared = PointThreshold * PointThreshold;
	for (const FBspIndexedPoint& IndexedPoint : GridItem.IndexedPoints)
	{
		if (FVector::DistSquared(IndexedPoint.Point, Point) <= PointThresholdSquared)
		{
			return IndexedPoint.Index;
		}
	}

	// Otherwise, the point is new: add it to the grid item.
	GridItem.IndexedPoints.Emplace(Point, Index);

	// The grid has a maximum threshold of a certain radius. If the point is near the edge of a grid cube, it may overlap into other items.
	// Add it to all grid items it can be seen from.
	const float GridThreshold = Threshold * OneOverGranularity;
	const int32 NeighbourX = GetAdjacentIndexIfOverlapping(GridIndexX, GridX, GridThreshold);
	const int32 NeighbourY = GetAdjacentIndexIfOverlapping(GridIndexY, GridY, GridThreshold);
	const int32 NeighbourZ = GetAdjacentIndexIfOverlapping(GridIndexZ, GridZ, GridThreshold);

	const bool bOverlapsInX = (NeighbourX != GridIndexX);
	const bool bOverlapsInY = (NeighbourY != GridIndexY);
	const bool bOverlapsInZ = (NeighbourZ != GridIndexZ);

	if (bOverlapsInX)
	{
		GridMap.FindOrAdd(FBspPointsKey(NeighbourX, GridIndexY, GridIndexZ)).IndexedPoints.Emplace(Point, Index);

		if (bOverlapsInY)
		{
			GridMap.FindOrAdd(FBspPointsKey(NeighbourX, NeighbourY, GridIndexZ)).IndexedPoints.Emplace(Point, Index);

			if (bOverlapsInZ)
			{
				GridMap.FindOrAdd(FBspPointsKey(NeighbourX, NeighbourY, NeighbourZ)).IndexedPoints.Emplace(Point, Index);
			}
		}
		else if (bOverlapsInZ)
		{
			GridMap.FindOrAdd(FBspPointsKey(NeighbourX, GridIndexY, NeighbourZ)).IndexedPoints.Emplace(Point, Index);
		}
	}
	else
	{
		if (bOverlapsInY)
		{
			GridMap.FindOrAdd(FBspPointsKey(GridIndexX, NeighbourY, GridIndexZ)).IndexedPoints.Emplace(Point, Index);

			if (bOverlapsInZ)
			{
				GridMap.FindOrAdd(FBspPointsKey(GridIndexX, NeighbourY, NeighbourZ)).IndexedPoints.Emplace(Point, Index);
			}
		}
		else if (bOverlapsInZ)
		{
			GridMap.FindOrAdd(FBspPointsKey(GridIndexX, GridIndexY, NeighbourZ)).IndexedPoints.Emplace(Point, Index);
		}
	}

	return Index;
}
