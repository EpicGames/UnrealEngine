// Copyright Epic Games, Inc. All Rights Reserved.

#include "BSPUtils.h"

#include "CoreMinimal.h"
#include "Misc/MemStack.h"
#include "Misc/FeedbackContext.h"
#include "Materials/MaterialInterface.h"
#include "Model.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Engine/Polys.h"
#include "BSPOps.h"
#include "Engine/Selection.h"
#include "Modules/ModuleManager.h"

#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "GameFramework/Actor.h"
#include "Engine/Brush.h"
//#include "Editor/EditorEngine.h"
//#include "EdMode.h"
//#include "EditorModeManager.h"
#include "SurfaceIterators.h"
//#include "ActorEditorUtils.h"
#include "Misc/FeedbackContext.h"
#include "EngineUtils.h"


IMPLEMENT_MODULE(FDefaultModuleImpl, BSPUtils)

DEFINE_LOG_CATEGORY_STATIC(LogBspUtils, Log, All);

/*---------------------------------------------------------------------------------------
	Globals.
---------------------------------------------------------------------------------------*/

// Magic numbers.
#define THRESH_OPTGEOM_COPLANAR			(0.25)		/* Threshold for Bsp geometry optimization */
#define THRESH_OPTGEOM_COSIDAL			(0.25)		/* Threshold for Bsp geometry optimization */

//
// Status of filtered polygons:
//
enum EPolyNodeFilter
{
	F_OUTSIDE				= 0, // Leaf is an exterior leaf (visible to viewers).
	F_INSIDE				= 1, // Leaf is an interior leaf (non-visible, hidden behind backface).
	F_COPLANAR_OUTSIDE		= 2, // Poly is coplanar and in the exterior (visible to viewers).
	F_COPLANAR_INSIDE		= 3, // Poly is coplanar and inside (invisible to viewers).
	F_COSPATIAL_FACING_IN	= 4, // Poly is coplanar, cospatial, and facing in.
	F_COSPATIAL_FACING_OUT	= 5, // Poly is coplanar, cospatial, and facing out.
};

//
// Generic filter function called by BspFilterEdPolys.  A and B are pointers
// to any integers that your specific routine requires (or NULL if not needed).
//
typedef void (*BSP_FILTER_FUNC)
(
	UModel			*Model,
	int32			iNode,
	FPoly			*EdPoly,
	EPolyNodeFilter Leaf,
	ENodePlace		ENodePlace
);

//
// Information used by FilterEdPoly.
//
class FCoplanarInfo
{
public:
	int32	iOriginalNode;
	int32	iBackNode;
	int		BackNodeOutside;
	int		FrontLeafOutside;
	int		ProcessingBack;
};

//
// Function to filter an EdPoly through the Bsp, calling a callback
// function for all chunks that fall into leaves.
//
void FilterEdPoly
(
	BSP_FILTER_FUNC FilterFunc, 
	UModel			*Model,
	int32  			iNode, 
	FPoly			*EdPoly, 
	FCoplanarInfo	CoplanarInfo, 
	int				Outside
);

//
// Bsp statistics used by link topic function.
//
class FBspStats
{
public:
	int   Polys;      // Number of BspSurfs.
	int   Nodes;      // Total number of Bsp nodes.
	int   MaxDepth;   // Maximum tree depth.
	int   AvgDepth;   // Average tree depth.
	int   Branches;   // Number of nodes with two children.
	int   Coplanars;  // Number of nodes holding coplanar polygons (in same plane as parent).
	int   Fronts;     // Number of nodes with only a front child.
	int   Backs;      // Number of nodes with only a back child.
	int   Leaves;     // Number of nodes with no children.
	int   FrontLeaves;// Number of leaf nodes that are in front of their parent.
	int   BackLeaves; // Number of leaf nodes that are in back of their parent.
	int   DepthCount; // Depth counter (used for finding average depth).
} GBspStats;

//
// Global variables shared between bspBrushCSG and AddWorldToBrushFunc.  These are very
// tightly tied into the function AddWorldToBrush, not general-purpose.
//
int32		GDiscarded;		// Number of polys discarded and not added.
int32		GNode;          // Node AddBrushToWorld is adding to.
int32		GLastCoplanar;	// Last coplanar beneath GNode at start of AddWorldToBrush.
int32		GNumNodes;		// Number of Bsp nodes at start of AddWorldToBrush.
UModel		*GModel;		// Level map Model we're adding to.

/*----------------------------------------------------------------------------
   EdPoly building and compacting.
----------------------------------------------------------------------------*/

//
// Trys to merge two polygons.  If they can be merged, replaces Poly1 and emptys Poly2
// and returns 1.  Otherwise, returns 0.
//
int TryToMerge( FPoly *Poly1, FPoly *Poly2 )
{
	// Find one overlapping point.
	int32 Start1=0, Start2=0;
	for( Start1=0; Start1<Poly1->Vertices.Num(); Start1++ )
		for( Start2=0; Start2<Poly2->Vertices.Num(); Start2++ )
			if( FVector3f::PointsAreSame(Poly1->Vertices[Start1], Poly2->Vertices[Start2]) )
				goto FoundOverlap;
	return 0;
	FoundOverlap:

	// Wrap around trying to merge.
	int32 End1  = Start1;
	int32 End2  = Start2;
	int32 Test1 = Start1+1; if (Test1>=Poly1->Vertices.Num()) Test1 = 0;
	int32 Test2 = Start2-1; if (Test2<0)                   Test2 = Poly2->Vertices.Num()-1;
	if( FVector3f::PointsAreSame(Poly1->Vertices[Test1],Poly2->Vertices[Test2]) )
	{
		End1   = Test1;
		Start2 = Test2;
	}
	else
	{
		Test1 = Start1-1; if (Test1<0)                   Test1=Poly1->Vertices.Num()-1;
		Test2 = Start2+1; if (Test2>=Poly2->Vertices.Num()) Test2=0;
		if( FVector3f::PointsAreSame(Poly1->Vertices[Test1],Poly2->Vertices[Test2]) )
		{
			Start1 = Test1;
			End2   = Test2;
		}
		else return 0;
	}

	// Build a new edpoly containing both polygons merged.
	FPoly NewPoly = *Poly1;
	NewPoly.Vertices.Empty();
	int32 Vertex = End1;
	for( int32 i=0; i<Poly1->Vertices.Num(); i++ )
	{
		NewPoly.Vertices.Add(Poly1->Vertices[Vertex]);
		if( ++Vertex >= Poly1->Vertices.Num() )
			Vertex=0;
	}
	Vertex = End2;
	for( int32 i=0; i<(Poly2->Vertices.Num()-2); i++ )
	{
		if( ++Vertex >= Poly2->Vertices.Num() )
			Vertex=0;
		NewPoly.Vertices.Add(Poly2->Vertices[Vertex]);
	}

	// Remove colinear vertices and check convexity.
	if( NewPoly.RemoveColinears() )
	{
		*Poly1 = NewPoly;
		Poly2->Vertices.Empty();
		return true;
	}
	else return 0;
}

//
// Merge all polygons in coplanar list that can be merged convexly.
//
void MergeCoplanars( UModel* Model, int32* PolyList, int32 PolyCount )
{
	int32 MergeAgain = 1;
	while( MergeAgain )
	{
		MergeAgain = 0;
		for( int32 i=0; i<PolyCount; i++ )
		{
			FPoly& Poly1 = Model->Polys->Element[PolyList[i]];
			if( Poly1.Vertices.Num() > 0 )
	        {
				for( int32 j=i+1; j<PolyCount; j++ )
				{
					FPoly& Poly2 = Model->Polys->Element[PolyList[j]];
					if( Poly2.Vertices.Num() > 0 )
					{
                  		if( TryToMerge( &Poly1, &Poly2 ) )
							MergeAgain=1;
					}
				}
			}
		}
	}
}

//
// Convert a Bsp node's polygon to an EdPoly, add it to the list, and recurse.
//
void MakeEdPolys( UModel* Model, int32 iNode, TArray<FPoly>* DestArray )
{
	FBspNode* Node = &Model->Nodes[iNode];

	FPoly Temp;
	if( FBSPUtils::bspNodeToFPoly(Model,iNode,&Temp) >= 3 )
	{
		DestArray->Add(Temp);
	}

	if( Node->iFront!=INDEX_NONE )
	{
		MakeEdPolys( Model, Node->iFront, DestArray );
	}
	if( Node->iBack !=INDEX_NONE ) 
	{
		MakeEdPolys( Model, Node->iBack, DestArray );
	}
	if( Node->iPlane!=INDEX_NONE )
	{
		MakeEdPolys( Model, Node->iPlane, DestArray );
	}
}

void FBSPUtils::bspBuildFPolys( UModel* Model, bool SurfLinks, int32 iNode, TArray<FPoly>* DestArray )
{
	if (DestArray == NULL)
	{
		DestArray = &Model->Polys->Element;
	}
	DestArray->Reset();

	if( Model->Nodes.Num() )
	{
		MakeEdPolys( Model, iNode, DestArray );
	}

	if( !SurfLinks )
	{
		const int32 ElementsCount = DestArray->Num();
		for( int32 i=0; i<ElementsCount; i++ )
		{
			(*DestArray)[i].iLink=i;
		}
	}
}

void FBSPUtils::bspMergeCoplanars( UModel* Model, bool RemapLinks, bool MergeDisparateTextures )
{
	int32 OriginalNum = Model->Polys->Element.Num();

	// Mark all polys as unprocessed.
	for( int32 i=0; i<Model->Polys->Element.Num(); i++ )
		Model->Polys->Element[i].PolyFlags &= ~PF_EdProcessed;

	// Find matching coplanars and merge them.
	FMemMark Mark(FMemStack::Get());
	int32* PolyList = new(FMemStack::Get(),Model->Polys->Element.Num())int32;
	int32 n=0;
	for( int32 i=0; i<Model->Polys->Element.Num(); i++ )
	{
		FPoly* EdPoly = &Model->Polys->Element[i];
		if( EdPoly->Vertices.Num()>0 && !(EdPoly->PolyFlags & PF_EdProcessed) )
		{
			int32 PolyCount         =  0;
			PolyList[PolyCount++] =  i;
			EdPoly->PolyFlags    |= PF_EdProcessed;
			for( int32 j=i+1; j<Model->Polys->Element.Num(); j++ )
	        {
				FPoly* OtherPoly = &Model->Polys->Element[j];
				if( OtherPoly->iLink == EdPoly->iLink && OtherPoly->Vertices.Num() )
				{
					float Dist = (OtherPoly->Vertices[0] - EdPoly->Vertices[0]) | EdPoly->Normal;
					if
					(	Dist>-0.001
					&&	Dist<0.001
					&&	(OtherPoly->Normal|EdPoly->Normal)>0.9999
					&&	(MergeDisparateTextures
						||	(	FVector::PointsAreNear((FVector)OtherPoly->TextureU, (FVector)EdPoly->TextureU,THRESH_VECTORS_ARE_NEAR)
							&&	FVector::PointsAreNear((FVector)OtherPoly->TextureV, (FVector)EdPoly->TextureV,THRESH_VECTORS_ARE_NEAR) ) ) )
					{
						OtherPoly->PolyFlags |= PF_EdProcessed;
						PolyList[PolyCount++] = j;
					}
				}
			}
			if( PolyCount > 1 )
	        {
				MergeCoplanars( Model, PolyList, PolyCount );
				n++;
			}
		}
	}
//	UE_LOGF(LogEditorBsp, Log,  "Found %i coplanar sets in %i", n, Model->Polys->Element.Num() );
	Mark.Pop();

	// Get rid of empty EdPolys while remapping iLinks.
	FMemMark Mark2(FMemStack::Get());
	int32 j=0;
	int32* Remap = new(FMemStack::Get(),Model->Polys->Element.Num())int32;
	for( int32 i=0; i<Model->Polys->Element.Num(); i++ )
	{
		if( Model->Polys->Element[i].Vertices.Num() )
		{
			Remap[i] = j;
			Model->Polys->Element[j] = Model->Polys->Element[i];
			j++;
		}
	}
	Model->Polys->Element.RemoveAt( j, Model->Polys->Element.Num()-j );
	if( RemapLinks )
	{
		for( int32 i=0; i<Model->Polys->Element.Num(); i++ )
		{
			if (Model->Polys->Element[i].iLink != INDEX_NONE)
			{
				CA_SUPPRESS(6001); // warning C6001: Using uninitialized memory '*Remap'.
				Model->Polys->Element[i].iLink = Remap[Model->Polys->Element[i].iLink];
			}
		}
	}
// 	UE_LOGF(LogEditorBsp, Log,  "BspMergeCoplanars reduced %i->%i", OriginalNum, Model->Polys->Element.Num() );
	Mark2.Pop();
}

/*----------------------------------------------------------------------------
   CSG types & general-purpose callbacks.
----------------------------------------------------------------------------*/

//
// Recursive worker function called by BspCleanup.
//
void CleanupNodes( UModel *Model, int32 iNode, int32 iParent )
{
	FBspNode *Node = &Model->Nodes[iNode];

	// Transactionally empty vertices of tag-for-empty nodes.
	Node->NodeFlags &= ~(NF_IsNew | NF_IsFront | NF_IsBack);

	// Recursively clean up front, back, and plane nodes.
	if( Node->iFront != INDEX_NONE ) CleanupNodes( Model, Node->iFront, iNode );
	if( Node->iBack  != INDEX_NONE ) CleanupNodes( Model, Node->iBack , iNode );
	if( Node->iPlane != INDEX_NONE ) CleanupNodes( Model, Node->iPlane, iNode );

	// Reload Node since the recusive call aliases it.
	Node = &Model->Nodes[iNode];

	// If this is an empty node with a coplanar, replace it with the coplanar.
	if( Node->NumVertices==0 && Node->iPlane!=INDEX_NONE )
	{
		FBspNode* PlaneNode = &Model->Nodes[ Node->iPlane ];

		// Stick our front, back, and parent nodes on the coplanar.
		if( (Node->Plane | PlaneNode->Plane) >= 0.0 )
		{
			PlaneNode->iFront  = Node->iFront;
			PlaneNode->iBack   = Node->iBack;
		}
		else
		{
			PlaneNode->iFront  = Node->iBack;
			PlaneNode->iBack   = Node->iFront;
		}

		if( iParent == INDEX_NONE )
		{
			// This node is the root.
			*Node                  = *PlaneNode;   // Replace root.
			PlaneNode->NumVertices = 0;            // Mark as unused.
		}
		else
		{
			// This is a child node.
			FBspNode *ParentNode = &Model->Nodes[iParent];

			if      ( ParentNode->iFront == iNode ) ParentNode->iFront = Node->iPlane;
			else if ( ParentNode->iBack  == iNode ) ParentNode->iBack  = Node->iPlane;
			else if ( ParentNode->iPlane == iNode ) ParentNode->iPlane = Node->iPlane;
			else UE_LOGF(LogBspUtils, Fatal, "CleanupNodes: Parent and child are unlinked");
		}
	}
	else if( Node->NumVertices == 0 && ( Node->iFront==INDEX_NONE || Node->iBack==INDEX_NONE ) )
	{
		// Delete empty nodes with no fronts or backs.
		// Replace empty nodes with only fronts.
		// Replace empty nodes with only backs.
		int32 iReplacementNode;
		if     ( Node->iFront != INDEX_NONE ) iReplacementNode = Node->iFront;
		else if( Node->iBack  != INDEX_NONE ) iReplacementNode = Node->iBack;
		else                                  iReplacementNode = INDEX_NONE;

		if( iParent == INDEX_NONE )
		{
			// Root.
			if( iReplacementNode == INDEX_NONE )
			{
         		Model->Nodes.Empty();
			}
			else
			{
         		*Node = Model->Nodes[iReplacementNode];
			}
		}
		else
		{
			// Regular node.
			FBspNode *ParentNode = &Model->Nodes[iParent];

			if     ( ParentNode->iFront == iNode ) ParentNode->iFront = iReplacementNode;
			else if( ParentNode->iBack  == iNode ) ParentNode->iBack  = iReplacementNode;
			else if( ParentNode->iPlane == iNode ) ParentNode->iPlane = iReplacementNode;
			else UE_LOGF(LogBspUtils, Fatal, "CleanupNodes: Parent and child are unlinked");
		}
	}
}


void FBSPUtils::bspCleanup( UModel *Model )
{
	if( Model->Nodes.Num() > 0 ) 
		CleanupNodes( Model, 0, INDEX_NONE );
}

/*----------------------------------------------------------------------------
   CSG leaf filter callbacks.
----------------------------------------------------------------------------*/

void AddBrushToWorldFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter, ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
			FBSPOps::bspAddNode (Model,iNode,ENodePlace,NF_IsNew,EdPoly);
			break;
		case F_COSPATIAL_FACING_OUT:
			if( !(EdPoly->PolyFlags & PF_Semisolid) )
				FBSPOps::bspAddNode (Model,iNode,ENodePlace,NF_IsNew,EdPoly);
			break;
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
		case F_COSPATIAL_FACING_IN:
			break;
	}
}

void AddWorldToBrushFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter, ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
			// Only affect the world poly if it has been cut.
			if( EdPoly->PolyFlags & PF_EdCut )
				FBSPOps::bspAddNode( GModel, GLastCoplanar, FBSPOps::NODE_Plane, NF_IsNew, EdPoly );
			break;
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
		case F_COSPATIAL_FACING_IN:
		case F_COSPATIAL_FACING_OUT:
			// Discard original poly.
			GDiscarded++;
			if( GModel->Nodes[GNode].NumVertices )
			{
				GModel->Nodes[GNode].NumVertices = 0;
			}
			break;
	}
}

void SubtractBrushFromWorldFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter, ENodePlace ENodePlace )
{
	switch (Filter)
	{
		case F_OUTSIDE:
		case F_COSPATIAL_FACING_OUT:
		case F_COSPATIAL_FACING_IN:
		case F_COPLANAR_OUTSIDE:
			break;
		case F_COPLANAR_INSIDE:
		case F_INSIDE:
			EdPoly->Reverse();
			FBSPOps::bspAddNode (Model,iNode,ENodePlace,NF_IsNew,EdPoly); // Add to Bsp back
			EdPoly->Reverse();
			break;
	}
}

void SubtractWorldToBrushFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter, ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
		case F_COSPATIAL_FACING_IN:
			// Only affect the world poly if it has been cut.
			if( EdPoly->PolyFlags & PF_EdCut )
				FBSPOps::bspAddNode( GModel, GLastCoplanar, FBSPOps::NODE_Plane, NF_IsNew, EdPoly );
			break;
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
		case F_COSPATIAL_FACING_OUT:
			// Discard original poly.
			GDiscarded++;
			if( GModel->Nodes[GNode].NumVertices )
			{
				GModel->Nodes[GNode].NumVertices = 0;
			}
			break;
	}
}

void IntersectBrushWithWorldFunc( UModel* Model, int32 iNode, FPoly *EdPoly,
	EPolyNodeFilter Filter,ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
		case F_COSPATIAL_FACING_IN:
		case F_COSPATIAL_FACING_OUT:
			// Ignore.
			break;
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
			if( EdPoly->Fix()>=3 )
				GModel->Polys->Element.Add(*EdPoly);
			break;
	}
}

void IntersectWorldWithBrushFunc( UModel *Model, int32 iNode, FPoly *EdPoly,
	EPolyNodeFilter Filter,ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
		case F_COSPATIAL_FACING_IN:
			// Ignore.
			break;
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
		case F_COSPATIAL_FACING_OUT:
			if( EdPoly->Fix() >= 3 )
				GModel->Polys->Element.Add(*EdPoly);
			break;
	}
}

void DeIntersectBrushWithWorldFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter,ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_INSIDE:
		case F_COPLANAR_INSIDE:
		case F_COSPATIAL_FACING_OUT:
		case F_COSPATIAL_FACING_IN:
			// Ignore.
			break;
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
			if( EdPoly->Fix()>=3 )
				GModel->Polys->Element.Add(*EdPoly);
			break;
	}
}

void DeIntersectWorldWithBrushFunc( UModel* Model, int32 iNode, FPoly* EdPoly,
	EPolyNodeFilter Filter,ENodePlace ENodePlace )
{
	switch( Filter )
	{
		case F_OUTSIDE:
		case F_COPLANAR_OUTSIDE:
		case F_COSPATIAL_FACING_OUT:
			// Ignore.
			break;
		case F_COPLANAR_INSIDE:
		case F_INSIDE:
		case F_COSPATIAL_FACING_IN:
			if( EdPoly->Fix() >= 3 )
	        {
				EdPoly->Reverse();
				GModel->Polys->Element.Add(*EdPoly);
				EdPoly->Reverse();
			}
			break;
	}
}

/*----------------------------------------------------------------------------
   CSG polygon filtering routine (calls the callbacks).
----------------------------------------------------------------------------*/

//
// Handle a piece of a polygon that was filtered to a leaf.
//
void FilterLeaf
(
	BSP_FILTER_FUNC FilterFunc, 
	UModel*			Model,
	int32			    iNode, 
	FPoly*			EdPoly, 
	FCoplanarInfo	CoplanarInfo, 
	int32				LeafOutside, 
	ENodePlace		ENodePlace
)
{
	EPolyNodeFilter FilterType;

	if( CoplanarInfo.iOriginalNode == INDEX_NONE )
	{
		// Processing regular, non-coplanar polygons.
		FilterType = LeafOutside ? F_OUTSIDE : F_INSIDE;
		FilterFunc( Model, iNode, EdPoly, FilterType, ENodePlace );
	}
	else if( CoplanarInfo.ProcessingBack )
	{
		// Finished filtering polygon through tree in back of parent coplanar.
		DoneFilteringBack:
		if      ((!LeafOutside) && (!CoplanarInfo.FrontLeafOutside)) FilterType = F_COPLANAR_INSIDE;
		else if (( LeafOutside) && ( CoplanarInfo.FrontLeafOutside)) FilterType = F_COPLANAR_OUTSIDE;
		else if ((!LeafOutside) && ( CoplanarInfo.FrontLeafOutside)) FilterType = F_COSPATIAL_FACING_OUT;
		else if (( LeafOutside) && (!CoplanarInfo.FrontLeafOutside)) FilterType = F_COSPATIAL_FACING_IN;
		else
		{
			UE_LOGF(LogBspUtils, Fatal, "FilterLeaf: Bad Locs");
			return;
		}
		FilterFunc( Model, CoplanarInfo.iOriginalNode, EdPoly, FilterType, FBSPOps::NODE_Plane );
	}
	else
	{
		CoplanarInfo.FrontLeafOutside = LeafOutside;

		if( CoplanarInfo.iBackNode == INDEX_NONE )
		{
			// Back tree is empty.
			LeafOutside = CoplanarInfo.BackNodeOutside;
			goto DoneFilteringBack;
		}
		else
		{
			// Call FilterEdPoly to filter through the back.  This will result in
			// another call to FilterLeaf with iNode = leaf this falls into in the
			// back tree and EdPoly = the final EdPoly to insert.
			CoplanarInfo.ProcessingBack=1;
			FilterEdPoly( FilterFunc, Model, CoplanarInfo.iBackNode, EdPoly,CoplanarInfo, CoplanarInfo.BackNodeOutside );
		}
	}
}

//
// Filter an EdPoly through the Bsp recursively, calling FilterFunc
// for all chunks that fall into leaves.  FCoplanarInfo is used to
// handle the tricky case of double-recursion for polys that must be
// filtered through a node's front, then filtered through the node's back,
// in order to handle coplanar CSG properly.
//
void FilterEdPoly
(
	BSP_FILTER_FUNC	FilterFunc, 
	UModel			*Model,
	int32			    iNode, 
	FPoly			*EdPoly, 
	FCoplanarInfo	CoplanarInfo, 
	int32				Outside
)
{
	int32            SplitResult,iOurFront,iOurBack;
	int32			   NewFrontOutside,NewBackOutside;

	FilterLoop:

	// Split em.
	FPoly TempFrontEdPoly,TempBackEdPoly;
	SplitResult = EdPoly->SplitWithPlane
	(
		Model->Points [Model->Verts[Model->Nodes[iNode].iVertPool].pVertex],
		Model->Vectors[Model->Surfs[Model->Nodes[iNode].iSurf].vNormal],
		&TempFrontEdPoly,
		&TempBackEdPoly,
		0
	);

	// Process split results.
	if( SplitResult == SP_Front )
	{
		Front:

		FBspNode *Node = &Model->Nodes[iNode];
		Outside        = Outside || Node->IsCsg();

		if( Node->iFront == INDEX_NONE )
		{
			FilterLeaf(FilterFunc,Model,iNode,EdPoly,CoplanarInfo,Outside,FBSPOps::NODE_Front);
		}
		else
		{
			iNode = Node->iFront;
			goto FilterLoop;
		}
	}
	else if( SplitResult == SP_Back )
	{
		FBspNode *Node = &Model->Nodes[iNode];
		Outside        = Outside && !Node->IsCsg();

		if( Node->iBack == INDEX_NONE )
		{
			FilterLeaf( FilterFunc, Model, iNode, EdPoly, CoplanarInfo, Outside, FBSPOps::NODE_Back );
		}
		else
		{
			iNode=Node->iBack;
			goto FilterLoop;
		}
	}
	else if( SplitResult == SP_Coplanar )
	{
		if( CoplanarInfo.iOriginalNode != INDEX_NONE )
		{
			// This will happen once in a blue moon when a polygon is barely outside the
			// coplanar threshold and is split up into a new polygon that is
			// is barely inside the coplanar threshold.  To handle this, just classify
			// it as front and it will be handled properly.
			FBSPOps::GErrors++;
// 			UE_LOGF(LogEditorBsp, Warning, "FilterEdPoly: Encountered out-of-place coplanar" );
			goto Front;
		}
		CoplanarInfo.iOriginalNode        = iNode;
		CoplanarInfo.iBackNode            = INDEX_NONE;
		CoplanarInfo.ProcessingBack       = 0;
		CoplanarInfo.BackNodeOutside      = Outside;
		NewFrontOutside                   = Outside;

		// See whether Node's iFront or iBack points to the side of the tree on the front
		// of this polygon (will be as expected if this polygon is facing the same
		// way as first coplanar in link, otherwise opposite).
		if( (FVector(Model->Nodes[iNode].Plane) | (FVector)EdPoly->Normal) >= 0.0 )
		{
			iOurFront = Model->Nodes[iNode].iFront;
			iOurBack  = Model->Nodes[iNode].iBack;
			
			if( Model->Nodes[iNode].IsCsg() )
	        {
				CoplanarInfo.BackNodeOutside = 0;
				NewFrontOutside              = 1;
			}
		}
		else
		{
			iOurFront = Model->Nodes[iNode].iBack;
			iOurBack  = Model->Nodes[iNode].iFront;
			
			if( Model->Nodes[iNode].IsCsg() )
	        {
				CoplanarInfo.BackNodeOutside = 1; 
				NewFrontOutside              = 0;
			}
		}

		// Process front and back.
		if ((iOurFront==INDEX_NONE)&&(iOurBack==INDEX_NONE))
		{
			// No front or back.
			CoplanarInfo.ProcessingBack		= 1;
			CoplanarInfo.FrontLeafOutside	= NewFrontOutside;
			FilterLeaf
			(
				FilterFunc,
				Model,
				iNode,
				EdPoly,
				CoplanarInfo,
				CoplanarInfo.BackNodeOutside,
				FBSPOps::NODE_Plane
			);
		}
		else if( iOurFront==INDEX_NONE && iOurBack!=INDEX_NONE )
		{
			// Back but no front.
			CoplanarInfo.ProcessingBack		= 1;
			CoplanarInfo.iBackNode			= iOurBack;
			CoplanarInfo.FrontLeafOutside	= NewFrontOutside;

			iNode   = iOurBack;
			Outside = CoplanarInfo.BackNodeOutside;
			goto FilterLoop;
		}
		else
		{
			// Has a front and maybe a back.

			// Set iOurBack up to process back on next call to FilterLeaf, and loop
			// to process front.  Next call to FilterLeaf will set FrontLeafOutside.
			CoplanarInfo.ProcessingBack = 0;

			// May be a node or may be INDEX_NONE.
			CoplanarInfo.iBackNode = iOurBack;

			iNode   = iOurFront;
			Outside = NewFrontOutside;
			goto FilterLoop;
		}
	}
	else if( SplitResult == SP_Split )
	{
		// Front half of split.
		if( Model->Nodes[iNode].IsCsg() )
		{
			NewFrontOutside = 1; 
			NewBackOutside  = 0;
		}
		else
		{
			NewFrontOutside = Outside;
			NewBackOutside  = Outside;
		}

		if( Model->Nodes[iNode].iFront==INDEX_NONE )
		{
			FilterLeaf
			(
				FilterFunc,
				Model,
				iNode,
				&TempFrontEdPoly,
				CoplanarInfo,
				NewFrontOutside,
				FBSPOps::NODE_Front
			);
		}
		else
		{
			FilterEdPoly
			(
				FilterFunc,
				Model,
				Model->Nodes[iNode].iFront,
				&TempFrontEdPoly,
				CoplanarInfo,
				NewFrontOutside
			);
		}

		// Back half of split.
		if( Model->Nodes[iNode].iBack==INDEX_NONE )
		{
			FilterLeaf
			(
				FilterFunc,
				Model,
				iNode,
				&TempBackEdPoly,
				CoplanarInfo,
				NewBackOutside,
				FBSPOps::NODE_Back
			);
		}
		else
		{
			FilterEdPoly
			(
				FilterFunc,
				Model,
				Model->Nodes[iNode].iBack,
				&TempBackEdPoly,
				CoplanarInfo,
				NewBackOutside
			);
		}
	}
}

//
// Regular entry into FilterEdPoly (so higher-level callers don't have to
// deal with unnecessary info). Filters starting at root.
//
void BspFilterFPoly( BSP_FILTER_FUNC FilterFunc, UModel *Model, FPoly *EdPoly )
{
	FCoplanarInfo StartingCoplanarInfo;
	StartingCoplanarInfo.iOriginalNode = INDEX_NONE;
	if( Model->Nodes.Num() == 0 )
	{
		// If Bsp is empty, process at root.
		FilterFunc( Model, 0, EdPoly, Model->RootOutside ? F_OUTSIDE : F_INSIDE, FBSPOps::NODE_Root );
	}
	else
	{
		// Filter through Bsp.
		FilterEdPoly( FilterFunc, Model, 0, EdPoly, StartingCoplanarInfo, Model->RootOutside );
	}
}


int FBSPUtils::bspNodeToFPoly
(
	UModel	*Model,
	int32	iNode,
	FPoly	*EdPoly
)
{
	FPoly BrushEdPoly;

	FBspNode &Node     	= Model->Nodes[iNode];
	FBspSurf &Poly     	= Model->Surfs[Node.iSurf];
	FVert	 *VertPool	= &Model->Verts[ Node.iVertPool ];

	EdPoly->Base		= Model->Points [Poly.pBase];
	EdPoly->Normal      = Model->Vectors[Poly.vNormal];

	EdPoly->PolyFlags 	= Poly.PolyFlags & ~(PF_EdCut | PF_EdProcessed | PF_Selected | PF_Memorized);
	EdPoly->iLinkSurf   = Node.iSurf;
	EdPoly->Material    = Poly.Material;

	EdPoly->Actor    	= Poly.Actor;
	EdPoly->iBrushPoly  = Poly.iBrushPoly;
	
	if( polyFindBrush(Model,Node.iSurf,BrushEdPoly) )
		EdPoly->ItemName  = BrushEdPoly.ItemName;
	else
		EdPoly->ItemName  = NAME_None;

	EdPoly->TextureU = Model->Vectors[Poly.vTextureU];
	EdPoly->TextureV = Model->Vectors[Poly.vTextureV];

	EdPoly->LightMapScale = Poly.LightMapScale;

 	EdPoly->LightmassSettings = Model->LightmassSettings[Poly.iLightmassIndex];

	EdPoly->Vertices.Empty();

	for(int32 VertexIndex = 0;VertexIndex < Node.NumVertices;VertexIndex++)
	{
		EdPoly->Vertices.Add(Model->Points[VertPool[VertexIndex].pVertex]);
	}

	if(EdPoly->Vertices.Num() < 3)
	{
		EdPoly->Vertices.Empty();
	}
	else
	{
		// Remove colinear points and identical points (which will appear
		// if T-joints were eliminated).
		EdPoly->RemoveColinears();
	}

	return EdPoly->Vertices.Num();
}

/*---------------------------------------------------------------------------------------
   World filtering.
---------------------------------------------------------------------------------------*/

//
// Filter all relevant world polys through the brush.
//
void FilterWorldThroughBrush 
(
	UModel*		Model,
	UModel*		Brush,
	EBrushType	BrushType,
	ECsgOper	CSGOper,
	int32		    iNode,
	FSphere*	BrushSphere
)
{
	// Loop through all coplanars.
	while( iNode != INDEX_NONE )
	{
		// Get surface.
		int32 iSurf = Model->Nodes[iNode].iSurf;

		// Skip new nodes and their children, which are guaranteed new.
		if( Model->Nodes[iNode].NodeFlags & NF_IsNew )
			return;

		// Sphere reject.
		int DoFront = 1, DoBack = 1;
		if( BrushSphere )
		{
			float Dist = Model->Nodes[iNode].Plane.PlaneDot( (FVector3f)BrushSphere->Center );
			DoFront    = (Dist >= -BrushSphere->W);
			DoBack     = (Dist <= +BrushSphere->W);
		}

		// Process only polys that aren't empty.
		FPoly TempEdPoly;
		if( DoFront && DoBack && (FBSPUtils::bspNodeToFPoly(Model,iNode,&TempEdPoly)>0) )
		{
			TempEdPoly.Actor      = Model->Surfs[iSurf].Actor;
			TempEdPoly.iBrushPoly = Model->Surfs[iSurf].iBrushPoly;

			if( BrushType==Brush_Add || BrushType==Brush_Subtract )
			{
				// Add and subtract work the same in this step.
				GNode       = iNode;
				GModel  	= Model;
				GDiscarded  = 0;
				GNumNodes	= Model->Nodes.Num();

				// Find last coplanar in chain.
				GLastCoplanar = iNode;
				while( Model->Nodes[GLastCoplanar].iPlane != INDEX_NONE )
					GLastCoplanar = Model->Nodes[GLastCoplanar].iPlane;

				// Do the filter operation.
				BspFilterFPoly
				(
					BrushType==Brush_Add ? AddWorldToBrushFunc : SubtractWorldToBrushFunc,
					Brush,
					&TempEdPoly
				);
				
				if( GDiscarded == 0 )
				{
					// Get rid of all the fragments we added.
					Model->Nodes[GLastCoplanar].iPlane = INDEX_NONE;
					Model->Nodes.RemoveAt( GNumNodes, Model->Nodes.Num()-GNumNodes, EAllowShrinking::No );
				}
				else
				{
					// Tag original world poly for deletion; has been deleted or replaced by partial fragments.
					if( GModel->Nodes[GNode].NumVertices )
					{
						GModel->Nodes[GNode].NumVertices = 0;
					}
				}
			}
			else if( CSGOper == CSG_Intersect )
			{
				BspFilterFPoly( IntersectWorldWithBrushFunc, Brush, &TempEdPoly );
			}
			else if( CSGOper == CSG_Deintersect )
			{
				BspFilterFPoly( DeIntersectWorldWithBrushFunc, Brush, &TempEdPoly );
			}
		}

		// Now recurse to filter all of the world's children nodes.
		if( DoFront && (Model->Nodes[iNode].iFront != INDEX_NONE)) FilterWorldThroughBrush
		(
			Model,
			Brush,
			BrushType,
			CSGOper,
			Model->Nodes[iNode].iFront,
			BrushSphere
		);
		if( DoBack && (Model->Nodes[iNode].iBack != INDEX_NONE) ) FilterWorldThroughBrush
		(
			Model,
			Brush,
			BrushType,
			CSGOper,
			Model->Nodes[iNode].iBack,
			BrushSphere
		);
		iNode = Model->Nodes[iNode].iPlane;
	}
}


int FBSPUtils::bspBrushCSG
(
	ABrush*		Actor,
	UModel*		Model,
	UModel*		TempModel,
	UMaterialInterface* SelectedMaterialInstance,
	uint32		PolyFlags, 
	EBrushType	BrushType,
	ECsgOper	CSGOper, 
	bool		bBuildBounds,
	bool		bMergePolys,
	bool		bReplaceNULLMaterialRefs,
	bool		bShowProgressBar/*=true*/
)
{
	uint32 NotPolyFlags = 0;
	int32 NumPolysFromBrush=0,i,j,ReallyBig;
	UModel* Brush = Actor->Brush;

	// Note no errors.
	FBSPOps::GErrors = 0;

	// Make sure we're in an acceptable state.
	if( !Brush )
	{
		return 0;
	}

	// Non-solid and semisolid stuff can only be added.
	if( BrushType != Brush_Add )
	{
		NotPolyFlags |= (PF_Semisolid | PF_NotSolid);
	}

	TempModel->EmptyModel(1,1);

	// Update status.
	ReallyBig = (Brush->Polys->Element.Num() > 200) && bShowProgressBar;
	if( ReallyBig )
	{
		FText Description = NSLOCTEXT("UnrealEd", "PerformingCSGOperation", "Performing CSG operation");
		
		if (BrushType != Brush_MAX)
		{	
			if (BrushType == Brush_Add)
			{
				Description = NSLOCTEXT("UnrealEd", "AddingBrushToWorld", "Adding brush to world");
			}
			else if (BrushType == Brush_Subtract)
			{
				Description = NSLOCTEXT("UnrealEd", "SubtractingBrushFromWorld", "Subtracting brush from world");
			}
		}
		else if (CSGOper != CSG_None)
		{
			if (CSGOper == CSG_Intersect)
			{
				Description = NSLOCTEXT("UnrealEd", "IntersectingBrushWithWorld", "Intersecting brush with world");
			}
			else if (CSGOper == CSG_Deintersect)
			{
				Description = NSLOCTEXT("UnrealEd", "DeintersectingBrushWithWorld", "Deintersecting brush with world");
			}
		}

		GWarn->BeginSlowTask( Description, true );
		// Transform original brush poly into same coordinate system as world
		// so Bsp filtering operations make sense.
		GWarn->StatusUpdate(0, 0, NSLOCTEXT("UnrealEd", "Transforming", "Transforming"));
	}

	const FVector Scale = Actor->GetActorScale();
	const FRotator Rotation = Actor->GetActorRotation();
	const FVector Location = Actor->GetActorLocation();

	const bool bIsMirrored = (Scale.X * Scale.Y * Scale.Z < 0.0f);

	// Cache actor transform which is used for the geometry being built
	Brush->OwnerLocationWhenLastBuilt = (FVector3f)Location;
	Brush->OwnerRotationWhenLastBuilt = Rotation;
	Brush->OwnerScaleWhenLastBuilt = (FVector3f)Scale;
	Brush->bCachedOwnerTransformValid = true;

	for( i=0; i<Brush->Polys->Element.Num(); i++ )
	{
		FPoly& CurrentPoly = Brush->Polys->Element[i];

		// Set texture the first time.
		if ( bReplaceNULLMaterialRefs )
		{
			auto& PolyMat = CurrentPoly.Material;
			if ( !PolyMat || PolyMat == UMaterial::GetDefaultMaterial(MD_Surface) )
			{
				PolyMat = SelectedMaterialInstance;
			}
		}

		// Get the brush poly.
		FPoly DestEdPoly = CurrentPoly;
		check(CurrentPoly.iLink<Brush->Polys->Element.Num());

		// Set its backward brush link.
		DestEdPoly.Actor       = Actor;
		DestEdPoly.iBrushPoly  = i;

		// Update its flags.
		DestEdPoly.PolyFlags = (DestEdPoly.PolyFlags | PolyFlags) & ~NotPolyFlags;

		// Set its internal link.
		if (DestEdPoly.iLink == INDEX_NONE)
		{
			DestEdPoly.iLink = i;
		}

		// Transform it.
		DestEdPoly.Scale( (FVector3f)Scale );
		DestEdPoly.Rotate( FRotator3f(Rotation) );	// LWC_TODO: Precision loss?
		DestEdPoly.Transform( (FVector3f)Location );	// LWC_TODO: Precision loss

		// Reverse winding and normal if the parent brush is mirrored
		if (bIsMirrored)
		{
			DestEdPoly.Reverse();
			DestEdPoly.CalcNormal();
		}

		// Add poly to the temp model.
		TempModel->Polys->Element.Add( DestEdPoly );
	}
	if( ReallyBig ) GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "FilteringBrush", "Filtering brush") );

	// Pass the brush polys through the world Bsp.
	if( CSGOper==CSG_Intersect || CSGOper==CSG_Deintersect )
	{
		// Empty the brush.
		Brush->EmptyModel(1,1);

		// Intersect and deintersect.
		for( i=0; i<TempModel->Polys->Element.Num(); i++ )
		{
         	FPoly EdPoly = TempModel->Polys->Element[i];
			GModel = Brush;
			// TODO: iLink / iLinkSurf in EdPoly / TempModel->Polys->Element[i] ?
			BspFilterFPoly( CSGOper==CSG_Intersect ? IntersectBrushWithWorldFunc : DeIntersectBrushWithWorldFunc, Model,  &EdPoly );
		}
		NumPolysFromBrush = Brush->Polys->Element.Num();
	}
	else
	{
		// Add and subtract.
		TMap<int32, int32> SurfaceIndexRemap;
		for( i=0; i<TempModel->Polys->Element.Num(); i++ )
		{
         	FPoly EdPoly = TempModel->Polys->Element[i];

         	// Mark the polygon as non-cut so that it won't be harmed unless it must
         	// be split, and set iLink so that BspAddNode will know to add its information
         	// if a node is added based on this poly.
         	EdPoly.PolyFlags &= ~(PF_EdCut);
			const int32* SurfaceIndexPtr = SurfaceIndexRemap.Find(EdPoly.iLink);
			if (SurfaceIndexPtr == nullptr)
			{
				const int32 NewSurfaceIndex = Model->Surfs.Num();
				SurfaceIndexRemap.Add(EdPoly.iLink, NewSurfaceIndex);
				EdPoly.iLinkSurf = TempModel->Polys->Element[i].iLinkSurf = NewSurfaceIndex;
			}
			else
			{
				EdPoly.iLinkSurf = TempModel->Polys->Element[i].iLinkSurf = *SurfaceIndexPtr;
			}

			// Filter brush through the world.
			BspFilterFPoly( BrushType==Brush_Add ? AddBrushToWorldFunc : SubtractBrushFromWorldFunc, Model, &EdPoly );
		}
	}
	if( Model->Nodes.Num() && !(PolyFlags & (PF_NotSolid | PF_Semisolid)) )
	{
		// Quickly build a Bsp for the brush, tending to minimize splits rather than balance
		// the tree.  We only need the cutting planes, though the entire Bsp struct (polys and
		// all) is built.

		FBspPointsGrid* LevelModelPointsGrid = FBspPointsGrid::GBspPoints;
		FBspPointsGrid* LevelModelVectorsGrid = FBspPointsGrid::GBspVectors;

		// For the bspBuild call, temporarily create a new pair of BspPointsGrids for the TempModel.
		TUniquePtr<FBspPointsGrid> BspPoints = MakeUnique<FBspPointsGrid>(50.0f, THRESH_POINTS_ARE_SAME);
 		TUniquePtr<FBspPointsGrid> BspVectors = MakeUnique<FBspPointsGrid>(1 / 16.0f, FMath::Max(THRESH_NORMALS_ARE_SAME, THRESH_VECTORS_ARE_NEAR));
 		FBspPointsGrid::GBspPoints = BspPoints.Get();
 		FBspPointsGrid::GBspVectors = BspVectors.Get();

		if( ReallyBig ) GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "BuildingBSP", "Building BSP") );
		
		FBSPOps::bspBuild( TempModel, FBSPOps::BSP_Lame, 0, 70, 1, 0 );

		// Reinstate the original BspPointsGrids used for building the level Model.
		FBspPointsGrid::GBspPoints = LevelModelPointsGrid;
		FBspPointsGrid::GBspVectors = LevelModelVectorsGrid;

		if( ReallyBig ) GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "FilteringWorld", "Filtering world") );
		GModel = Brush;
		TempModel->BuildBound();

		FSphere	BrushSphere = TempModel->Bounds.GetSphere();
		FilterWorldThroughBrush( Model, TempModel, BrushType, CSGOper, 0, &BrushSphere );
	}
	if( CSGOper==CSG_Intersect || CSGOper==CSG_Deintersect )
	{
		if( ReallyBig ) GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "AdjustingBrush", "Adjusting brush") );

		// Link polys obtained from the original brush.
		for( i=NumPolysFromBrush-1; i>=0; i-- )
		{
			FPoly *DestEdPoly = &Brush->Polys->Element[i];
			for( j=0; j<i; j++ )
			{
				if( DestEdPoly->iLink == Brush->Polys->Element[j].iLink )
				{
					DestEdPoly->iLink = j;
					break;
				}
			}
			if( j >= i ) DestEdPoly->iLink = i;
		}

		// Link polys obtained from the world.
		for( i=Brush->Polys->Element.Num()-1; i>=NumPolysFromBrush; i-- )
		{
			FPoly *DestEdPoly = &Brush->Polys->Element[i];
			for( j=NumPolysFromBrush; j<i; j++ )
			{
				if( DestEdPoly->iLink == Brush->Polys->Element[j].iLink )
				{
					DestEdPoly->iLink = j;
					break;
				}
			}
			if( j >= i ) DestEdPoly->iLink = i;
		}
		Brush->Linked = 1;

		// Detransform the obtained brush back into its original coordinate system.
		for( i=0; i<Brush->Polys->Element.Num(); i++ )
		{
			FPoly *DestEdPoly = &Brush->Polys->Element[i];
			DestEdPoly->Transform((FVector3f)-Location);
			DestEdPoly->Rotate(FRotator3f(Rotation.GetInverse()));
			DestEdPoly->Scale(FVector3f(1.0f) / (FVector3f)Scale);
			DestEdPoly->Fix();
			DestEdPoly->Actor		= NULL;
			DestEdPoly->iBrushPoly	= i;
		}
	}

	if( BrushType==Brush_Add || BrushType==Brush_Subtract )
	{
		// Clean up nodes, reset node flags.
		bspCleanup( Model );

		// Rebuild bounding volumes.
		if( bBuildBounds )
		{
			FBSPOps::bspBuildBounds( Model );
		}
	}

	Brush->NumUniqueVertices = TempModel->Points.Num();
	// Release TempModel.
	TempModel->EmptyModel(1,1);
	
	// Merge coplanars if needed.
	if( CSGOper==CSG_Intersect || CSGOper==CSG_Deintersect )
	{
		if( ReallyBig )
		{
			GWarn->StatusUpdate( 0, 0, NSLOCTEXT("UnrealEd", "Merging", "Merging") );
		}
		if( bMergePolys )
		{
			bspMergeCoplanars( Brush, 1, 0 );
		}
	}
	if( ReallyBig )
	{
		GWarn->EndSlowTask();
	}

	return 1 + FBSPOps::GErrors;
}

/*---------------------------------------------------------------------------------------
   Functions for maintaining linked geometry lists.
---------------------------------------------------------------------------------------*/

//
// A node and vertex number corresponding to a point, used in generating Bsp side links.
//
class FPointVert 
{
public:
	int32		    iNode;
	int32		    nVertex;
	FPointVert* Next;
};

//
// A list of point/vertex links, used in generating Bsp side links.
//
class FPointVertList
{
public:

	// Variables.
	UModel		*Model;
	FPointVert	**Index;
	FMemMark*	Mark;

	/** Initialization constructor. */
	FPointVertList()
	:	Model(NULL)
	,	Index(NULL)
	,	Mark(NULL)
	{}

	/** Destructor. */
	~FPointVertList()
	{
		check(!Mark);
	}

	// Allocate.
	void Alloc( UModel *ThisModel )
	{
		check(!Mark);
		Mark = new FMemMark(FMemStack::Get());

		// Allocate working tables.
		Model = ThisModel;
		Index = new(FMemStack::Get(),MEM_Zeroed,Model->Points.Num())FPointVert*;

	}

	// Free.
	void Free()
	{
		delete Mark;
		Mark = NULL;
	}

	// Add all of a node's vertices to a node-vertex list.
	void AddNode( int32 iNode )
	{
		FBspNode&	Node 	 =  Model->Nodes[iNode];
		FVert*		VertPool =  &Model->Verts[Node.iVertPool];

		for( int32 i=0; i < Node.NumVertices; i++ )
		{
			int32 pVertex = VertPool[i].pVertex;

			// Add new point/vertex pair to array, and insert new array entry
			// between index and first entry.
			FPointVert *PointVert	= new(FMemStack::Get())FPointVert;
			
			PointVert->iNode 		= iNode;
			PointVert->nVertex		= i;
			PointVert->Next			= Index[pVertex];

			Index[pVertex]			= PointVert;
		}
	}

	// Add all nodes' vertices in the model to a node-vertex list.
	void AddAllNodes()
	{
		for( int32 iNode=0; iNode < Model->Nodes.Num(); iNode++ )
			AddNode( iNode );

	}

	// Remove all of a node's vertices from a node-vertex list.
	void RemoveNode( int32 iNode )
	{
		FBspNode&	Node 		= Model->Nodes[iNode];
		FVert*		VertPool	= &Model->Verts[Node.iVertPool];

		// Loop through all of the node's vertices and search through the
		// corresponding point's node-vert list, and delink this node.
		int Count=0;
		for( int32 i = 0; i < Node.NumVertices; i++ )
		{
			int32 pVertex = VertPool[i].pVertex;

			for( FPointVert **PrevLink = &Index[pVertex]; *PrevLink; PrevLink=&(*PrevLink)->Next )
			{
				if( (*PrevLink)->iNode == iNode )
				{
					// Delink this entry from the list.
					*PrevLink = (*PrevLink)->Next;
					Count++;

					if( *PrevLink == NULL )
						break;
				}
			}

			// Node's vertex wasn't found, there's a bug.
			check(Count>=1);
		}
	}
};

/*---------------------------------------------------------------------------------------
   Geometry optimization.
---------------------------------------------------------------------------------------*/

//
// Add a point to a Bsp node before a specified vertex (between it and the previous one).
// VertexNumber can be from 0 (before first) to Node->NumVertices (after last).
//
// Splits node into two coplanar polys if necessary. If the polygon is split, the
// vertices will be distributed among this node and it's newly-linked iPlane node
// in an arbitrary way, that preserves the clockwise orientation of the vertices.
//
// Maintains node-vertex list, if not NULL.
//
void AddPointToNode
(
	UModel*         Model,
	FPointVertList* PointVerts,
	int32			    iNode, 
	int32			    VertexNumber, 
	int32			    pVertex 
)
{
	FBspNode&	Node = Model->Nodes[iNode];

	if( Node.NumVertices >= FBspNode::MAX_NODE_VERTICES - 1 )
	{
		// Just refuse to add point: This is a non-fatal problem.
//		UE_LOGF(LogEditorBsp, Warning, "Node side limit reached" );
		return;
	}

	// Remove node from vertex list, since vertex numbers will be reordered.
	if( PointVerts )
		PointVerts->RemoveNode( iNode );

	int32 iOldVert = Node.iVertPool;

	Node.iVertPool = Model->Verts.AddUninitialized( Node.NumVertices+1 );

	// Make sure this node doesn't already contain the vertex.
	for( int32 i=0; i<Node.NumVertices; i++ )
		check( Model->Verts[iOldVert + i].pVertex != pVertex );

	// Copy the old vertex pool to the new one.
	for( int32 i=0; i<VertexNumber; i++ )
		Model->Verts[Node.iVertPool + i] = Model->Verts[iOldVert + i];

	for( int32 i=VertexNumber; i<Node.NumVertices; i++ )
		Model->Verts[Node.iVertPool + i + 1] = Model->Verts[iOldVert + i];

	// Add the new point to the new vertex pool.
	Model->Verts[Node.iVertPool + VertexNumber].pVertex = pVertex;
	Model->Verts[Node.iVertPool + VertexNumber].iSide   = INDEX_NONE;

	// Increment number of node vertices.
	Node.NumVertices++;

	// Update the point-vertex list.
	if( PointVerts )
		PointVerts->AddNode( iNode );

}

//
// Add a point to all sides of polygons in which the side intersects with
// this point but doesn't contain it, and has the correct (clockwise) orientation
// as this side.  pVertex is the index of the point to handle, and
// ReferenceVertex defines the direction of this side.
//
int DistributePoint
(
	UModel*         Model,
	FPointVertList* PointVerts,	
	int32			    iNode,
	int32			    pVertex
)
{
	int32 Count = 0;

	// Handle front, back, and plane.
	float Dist = Model->Nodes[iNode].Plane.PlaneDot(Model->Points[pVertex]);
	if( Dist < THRESH_OPTGEOM_COPLANAR )
	{
		// Back.
		if( Model->Nodes[iNode].iBack != INDEX_NONE )
			Count += DistributePoint( Model, PointVerts, Model->Nodes[iNode].iBack, pVertex );
	}
	if( Dist > -THRESH_OPTGEOM_COPLANAR )
	{
		// Front.
		if( Model->Nodes[iNode].iFront != INDEX_NONE )
			Count += DistributePoint( Model, PointVerts, Model->Nodes[iNode].iFront, pVertex );
	}
	if( Dist > -THRESH_OPTGEOM_COPLANAR && Dist < THRESH_OPTGEOM_COPLANAR )
	{
		// This point is coplanar with this node, so check point for intersection with
		// this node's sides, then loop with its coplanars.
		for( ; iNode!=INDEX_NONE; iNode=Model->Nodes[iNode].iPlane )
		{
			FVert* VertPool = &Model->Verts[Model->Nodes[iNode].iVertPool];

			// Skip this node if it already contains the point in question.
			int32 i;
			for( i=0; i<Model->Nodes[iNode].NumVertices; i++ )
				if( VertPool[i].pVertex == pVertex )
					break;
			if( i != Model->Nodes[iNode].NumVertices )
				continue;

			// Loop through all sides and see if (A) side is colinear with point, and
			// (B) point falls within inside of this side.
			int32 FoundSide       = -1;
			int32 SkippedColinear = 0;
			int32 SkippedInside   = 0;

			for( i=0; i<Model->Nodes[iNode].NumVertices; i++ )
			{
				int32 j = (i>0) ? (i-1) : (Model->Nodes[iNode].NumVertices-1);

				// Create cutting plane perpendicular to both this side and the polygon's normal.
				FVector Side            = FVector(Model->Points[VertPool[i].pVertex] - Model->Points[VertPool[j].pVertex]);
				FVector SidePlaneNormal = Side ^ (FVector)Model->Nodes[iNode].Plane;
				const float SizeSquared = static_cast<float>(SidePlaneNormal.SizeSquared());

				if( SizeSquared>FMath::Square(0.001) )
				{
					// Points aren't coincedent.
					Dist = static_cast<float>((FVector(Model->Points[pVertex] - Model->Points[VertPool[i].pVertex]) | SidePlaneNormal) / FMath::Sqrt(SizeSquared));
					if( Dist >= THRESH_OPTGEOM_COSIDAL )
					{
						// Point is outside polygon, can't possibly fall on a side.
						break;
					}
					else if( Dist > -THRESH_OPTGEOM_COSIDAL )
					{
						// The point we're adding falls on this line.
						//
						// Verify that it falls within this side; though it's colinear
						// it may be out of the bounds of the line's endpoints if this side
						// is colinear with an adjacent side.
						//
						// Do this by checking distance from point to side's midpoint and
						// comparing with the side's half-length.

						FVector MidPoint    = FVector(Model->Points[VertPool[i].pVertex] + Model->Points[VertPool[j].pVertex])*0.5;
						FVector MidDistVect = (FVector)Model->Points[pVertex] - MidPoint;
						if( MidDistVect.SizeSquared() <= FMath::Square(0.501)*Side.SizeSquared() )
						{
							FoundSide = i;
						}
						else
						{
							SkippedColinear = 1;
						}
					}
					else
					{
						// Point is inside polygon, so continue checking.
						SkippedInside = 1;
					}
				}
				else
				{
					FBSPOps::GErrors++;
					//UE_LOGF(LogEditorBsp, Warning, "Found tiny side" );
					//Poly.PolyFlags |= PF_Selected;
				}
			}
			if( i==Model->Nodes[iNode].NumVertices && FoundSide>=0 )
			{
				// AddPointToNode will reorder the vertices in this node.  This is okay
				// because it's called outside of the vertex loop.
				AddPointToNode( Model, PointVerts, iNode, FoundSide, pVertex );
				Count++;
			}
			else if( SkippedColinear )
			{
				// This happens occasionally because of the fuzzy Dist comparison.  It is
				// not a sign of a problem when the vertex being distributed is colinear
				// with one of this polygon's sides, but slightly outside of this polygon.

				FBSPOps::GErrors++;
				//UE_LOGF(LogEditorBsp, Warning, "Skipped colinear" );
				//Poly.PolyFlags |= PF_Selected;
			}
			else if( SkippedInside )
			{
				// Point is on interior of polygon.
				FBSPOps::GErrors++;
				//UE_LOGF(LogEditorBsp, Warning, "Skipped interior" );
				//Poly.PolyFlags |= PF_Selected;
			}
		}
	}
	return Count;
}

//
// Merge points that are near.
//
void MergeNearPoints( UModel *Model, float Dist )
{
	FMemMark Mark(FMemStack::Get());
	int32* PointRemap = new(FMemStack::Get(),Model->Points.Num())int32;
	int32 Merged=0,Collapsed=0;

	// Find nearer point for all points.
	for( int32 i=0; i<Model->Points.Num(); i++ )
	{
		PointRemap[i] = i;
		FVector3f &Point = Model->Points[i];
		for( int32 j=0; j<i; j++ )
		{
			FVector3f &TestPoint = Model->Points[j];
			if( (TestPoint - Point).SizeSquared() < Dist*Dist )
			{
				PointRemap[i] = j;
				Merged++;
				break;
			}
		}
	}

	// Remap VertPool.
	for( int32 i=0; i<Model->Verts.Num(); i++ )
	{
		if( Model->Verts[i].pVertex>=0 && Model->Verts[i].pVertex<Model->Points.Num() )
		{
			CA_SUPPRESS(6001); // warning C6001: Using uninitialized memory '*PointRemap'.
			Model->Verts[i].pVertex = PointRemap[Model->Verts[i].pVertex];
		}
	}

	// Remap Surfs.
	for( int32 i=0; i<Model->Surfs.Num(); i++ )
	{
		if( Model->Surfs[i].pBase>=0 && Model->Surfs[i].pBase<Model->Points.Num() )
		{
			CA_SUPPRESS(6001); // warning C6001: Using uninitialized memory '*PointRemap'.
			Model->Surfs[i].pBase = PointRemap[Model->Surfs[i].pBase];
		}
	}

	// Remove duplicate points from nodes.
	for( int32 i=0; i<Model->Nodes.Num(); i++ )
	{
		FBspNode &Node = Model->Nodes[i];
		FVert *Pool = &Model->Verts[Node.iVertPool];
		
		int k=0;
		for( int j=0; j<Node.NumVertices; j++ )
		{
			FVert *A = &Pool[j];
			FVert *B = &Pool[j ? j-1 : Node.NumVertices-1];
			
			if( A->pVertex != B->pVertex )
				Pool[k++] = Pool[j];
		}
		Node.NumVertices = static_cast<uint8>(k>=3 ? k : 0);
		if( k < 3 )
			Collapsed++;
	}

	// Success.
// 	UE_LOGF(LogEditorBsp, Log,  "MergeNearPoints merged %i/%i, collapsed %i", Merged, Model->Points.Num(), Collapsed );
	Mark.Pop();
}


void FBSPUtils::bspOptGeom( UModel *Model )
{
	FPointVertList PointVerts;

	//UE_LOGF(LogEditorBsp, Log,  "BspOptGeom begin" );

	MergeNearPoints			(Model,0.25);
	FBSPOps::bspRefresh		(Model,0);
	PointVerts.Alloc		(Model);
	PointVerts.AddAllNodes	();

	// First four entries are reserved for view-clipped sides.
	Model->NumSharedSides = 4;
	
	// Mark all sides as unlinked.
	int32 i;
	for( i=0; i<Model->Verts.Num(); i++ )
		Model->Verts[i].iSide = INDEX_NONE;

	int32 TeesFound = 0, Distributed = 0;

	// Eliminate T-joints on each node by finding all vertices that aren't attached to
	// two shared sides, then filtering them down through the BSP and adding them to
	// the sides they belong on.
	for( int32 iNode=0; iNode < Model->Nodes.Num(); iNode++ )
	{
		FBspNode &Node = Model->Nodes[iNode];
		
		// Loop through all sides (side := line from PrevVert to ThisVert)		
		for( uint8 ThisVert=0; ThisVert < Node.NumVertices; ThisVert++ )
		{
			uint8 PrevVert = (ThisVert>0) ? (ThisVert - 1) : (Node.NumVertices-1);

			// Count number of nodes sharing this side, i.e. number of nodes for
			// which two adjacent vertices are identical to this side's two vertices.
			for( FPointVert* PV1 = PointVerts.Index[Model->Verts[Node.iVertPool+ThisVert].pVertex]; PV1; PV1=PV1->Next )
				for( FPointVert* PV2 = PointVerts.Index[Model->Verts[Node.iVertPool+PrevVert].pVertex]; PV2; PV2=PV2->Next )
					if( PV1->iNode==PV2->iNode && PV1->iNode!=iNode )
						goto SkipIt;

			// Didn't find another node that shares our two vertices; must add each
			// vertex to all polygons where the vertex lies on the polygon's side.
			// DistributePoint will not affect the current node but may change others
			// and may increase the number of nodes in the Bsp.
			TeesFound++;
			Distributed = 0;
			Distributed += DistributePoint( Model, &PointVerts, 0, Model->Verts[Node.iVertPool+ThisVert].pVertex );
			Distributed += DistributePoint( Model, &PointVerts, 0, Model->Verts[Node.iVertPool+PrevVert].pVertex );
			SkipIt:;
		}
	}
	// Build side links
	// Definition of side: Side (i) links node vertex (i) to vertex ((i+1)%n)
	//UE_LOGF(LogEditorBsp, Log,  "BspOptGeom building sidelinks" );

	PointVerts.Free			();
	PointVerts.Alloc		(Model);
	PointVerts.AddAllNodes	();

	for( int32 iNode=0; iNode < Model->Nodes.Num(); iNode++ )
	{
		FBspNode &Node = Model->Nodes[iNode];		
		for( uint8 ThisVert=0; ThisVert < Node.NumVertices; ThisVert++ )
		{
			if( Model->Verts[Node.iVertPool+ThisVert].iSide == INDEX_NONE )
			{
				// See if this node links to another one.
				uint8 PrevVert = (ThisVert>0) ? (ThisVert - 1) : (Node.NumVertices-1);
				for( FPointVert* PV1=PointVerts.Index[Model->Verts[Node.iVertPool+ThisVert].pVertex]; PV1; PV1=PV1->Next )
				{
					for( FPointVert* PV2=PointVerts.Index[Model->Verts[Node.iVertPool+PrevVert].pVertex]; PV2; PV2=PV2->Next )
					{
						if( PV1->iNode==PV2->iNode && PV1->iNode!=iNode )
						{
							// Make sure that the other node's two vertices are adjacent and
							// ordered opposite this node's vertices.
							int32		    iOtherNode	= PV2->iNode;
							FBspNode	&OtherNode	= Model->Nodes[iOtherNode];

							int32 Delta = 
								(OtherNode.NumVertices + PV2->nVertex - PV1->nVertex) % 
								OtherNode.NumVertices;

							if( Delta==1 )
							{
								// Side is properly linked!
								int32 OtherVert = PV2->nVertex;
								int32 iSide;
								if( Model->Verts[OtherNode.iVertPool+OtherVert].iSide==INDEX_NONE )
									iSide = Model->NumSharedSides++;
								else
									iSide = Model->Verts[OtherNode.iVertPool+OtherVert].iSide;
									
								// Link both sides to the shared side.
								Model->Verts[ Node.iVertPool + ThisVert ].iSide
								=	Model->Verts[ OtherNode.iVertPool+OtherVert ].iSide
								=	iSide;
								goto SkipSide;
							}
						}
					}
				}

				// This node doesn't have correct side linking
				//Model->Surfs(Node.iSurf).PolyFlags |= PF_Selected;
				FBSPOps::GErrors++;
				//UE_LOGF(LogEditorBsp, Warning, "Failed to link side" );
			}

			// Go to next side.
			SkipSide:;
		}
	}
	// Gather stats.
	i=0; int j=0;
	for( int32 iNode=0; iNode < Model->Nodes.Num(); iNode++ )
	{
		FBspNode&	Node		= Model->Nodes[iNode];
		FVert*		VertPool	= &Model->Verts[Node.iVertPool];
		
		i += Node.NumVertices;
		for( uint8 ThisVert=0; ThisVert < Node.NumVertices; ThisVert++ )
			if( VertPool[ThisVert].iSide!=INDEX_NONE )
				j++;
	}
	// Done.
	//UE_LOGF(LogEditorBsp, Log,  "BspOptGeom end" );
// 	UE_LOGF(LogEditorBsp, Log,  "Processed %i T-points, linked: %i/%i sides", TeesFound, j, i );

	PointVerts.Free();

	// Remove unused vertices from the vertex streams.
	// This is necessary to ensure the vertices added to eliminate T junctions
	// don't overflow the 65536 vertex/stream limit.

	FBSPOps::bspRefresh(Model,0);
}


void FBSPUtils::bspRepartition( UWorld* InWorld, int32 iNode )
{
	bspBuildFPolys( InWorld->GetModel(), 1, iNode );
	bspMergeCoplanars( InWorld->GetModel(), 0, 0 );
	FBSPOps::bspBuild( InWorld->GetModel(), FBSPOps::BSP_Good, 12, 70, 2, iNode );
	FBSPOps::bspRefresh( InWorld->GetModel(), 1 );
}


void FBSPUtils::polySetAndClearPolyFlags(UModel *Model, uint32 SetBits, uint32 ClearBits,bool SelectedOnly, bool UpdateBrush)
{
	for( int32 i=0; i<Model->Surfs.Num(); i++ )
	{
		FBspSurf& Poly = Model->Surfs[i];
		if( !SelectedOnly || (Poly.PolyFlags & PF_Selected) )
		{
			uint32 NewFlags = (Poly.PolyFlags & ~ClearBits) | SetBits;
			if( NewFlags != Poly.PolyFlags )
			{
				Model->ModifySurf( i, UpdateBrush );
				Poly.PolyFlags = NewFlags;
				if (UpdateBrush)
				{
					const bool bUpdateTexCoords = false;
					const bool bOnlyRefreshSurfaceMaterials = false;
					polyUpdateBrush(Model, i, bUpdateTexCoords, bOnlyRefreshSurfaceMaterials);
				}
			}
		}
	}
}


bool FBSPUtils::polyFindMaster(UModel* InModel, int32 iSurf, FPoly &Poly)
{
	return polyFindBrush(InModel, iSurf, Poly);
}

bool FBSPUtils::polyFindBrush(UModel* InModel, int32 iSurf, FPoly &Poly)
{
	FBspSurf &Surf = InModel->Surfs[iSurf];
	if( !Surf.Actor || !Surf.Actor->Brush->Polys->Element.IsValidIndex(Surf.iBrushPoly) )
	{
		return false;
	}
	else
	{
		Poly = Surf.Actor->Brush->Polys->Element[Surf.iBrushPoly];
		return true;
	}
}


void FBSPUtils::polyUpdateMaster(UModel* Model, int32 iSurf, bool bUpdateTexCoords,	bool bOnlyRefreshSurfaceMaterials)
{
	polyUpdateBrush(Model, iSurf, bUpdateTexCoords,	bOnlyRefreshSurfaceMaterials);
}

void FBSPUtils::polyUpdateBrush
(
	UModel*	Model,
	int32  	iSurf,
	bool	bUpdateTexCoords,
	bool	bOnlyRefreshSurfaceMaterials
)
{
	FBspSurf &Surf = Model->Surfs[iSurf];
	ABrush* Actor = Surf.Actor;
	if( !Actor )
		return;

	UModel* Brush = Actor->Brush;
	check(Brush);

	FVector ActorLocation;
	FVector ActorScale;
	FRotator ActorRotation;

	if (Brush->bCachedOwnerTransformValid)
	{
		// Use transform cached when the geometry was last built, in case the current Actor transform has changed since then
		// (e.g. because Auto Update BSP is disabled)
		ActorLocation = (FVector)Brush->OwnerLocationWhenLastBuilt;
		ActorScale = (FVector)Brush->OwnerScaleWhenLastBuilt;
		ActorRotation = Brush->OwnerRotationWhenLastBuilt;
	}
	else
	{
		// No cached owner transform, so use the current one
		ActorLocation = Actor->GetActorLocation();
		ActorScale = Actor->GetActorScale();
		ActorRotation = Actor->GetActorRotation();
	}

	const FRotationMatrix RotationMatrix(ActorRotation);

	for (int32 iEdPoly = Surf.iBrushPoly; iEdPoly < Brush->Polys->Element.Num(); iEdPoly++)
	{
		FPoly& BrushEdPoly = Brush->Polys->Element[iEdPoly];
		if (iEdPoly == Surf.iBrushPoly || BrushEdPoly.iLink == Surf.iBrushPoly)
		{
			BrushEdPoly.Material = Surf.Material;
			BrushEdPoly.PolyFlags = Surf.PolyFlags & ~(PF_NoEdit);

			if (bUpdateTexCoords)
			{
				BrushEdPoly.Base = FVector3f(RotationMatrix.InverseTransformVector((FVector)Model->Points[Surf.pBase] - ActorLocation) / ActorScale);
				BrushEdPoly.TextureU = FVector3f(RotationMatrix.InverseTransformVector((FVector)Model->Vectors[Surf.vTextureU]) * ActorScale);
				BrushEdPoly.TextureV = FVector3f(RotationMatrix.InverseTransformVector((FVector)Model->Vectors[Surf.vTextureV]) * ActorScale);
			}
		}
	}

	Model->InvalidSurfaces = true;

	if (bOnlyRefreshSurfaceMaterials)
	{
		Model->bOnlyRebuildMaterialIndexBuffers = true;
	}
}


void FBSPUtils::polyGetLinkedPolys
(
	ABrush* InBrush,
	FPoly* InPoly,
	TArray<FPoly>* InPolyList
)
{
	InPolyList->Empty();

	if( InPoly->iLink == INDEX_NONE )
	{
		// If this poly has no links, just stick the one poly in the final list.
		InPolyList->Add( *InPoly );
	}
	else
	{
		// Find all polys that match the source polys link value.
		for( int32 poly = 0 ; poly < InBrush->Brush->Polys->Element.Num() ; poly++ )
			if( InBrush->Brush->Polys->Element[poly].iLink == InPoly->iLink )
				InPolyList->Add( InBrush->Brush->Polys->Element[poly] );
	}
}


void FBSPUtils::polySplitOverlappingEdges( TArray<FPoly>* InPolyList, TArray<FPoly>* InResult )
{
	InResult->Empty();

	for( int32 poly = 0 ; poly < InPolyList->Num() ; poly++ )
	{
		FPoly* SrcPoly = &(*InPolyList)[poly];
		FPoly NewPoly = *SrcPoly;

		for( int32 edge = 0 ; edge < SrcPoly->Vertices.Num() ; edge++ )
		{
			FEdge SrcEdge = FEdge( (FVector)SrcPoly->Vertices[edge], (FVector)SrcPoly->Vertices[ edge+1 < SrcPoly->Vertices.Num() ? edge+1 : 0 ] );
			FPlane SrcEdgePlane( SrcEdge.Vertex[0], SrcEdge.Vertex[1], SrcEdge.Vertex[0] + (FVector)(SrcPoly->Normal * 16) );

			for( int32 poly2 = 0 ; poly2 < InPolyList->Num() ; poly2++ )
			{
				FPoly* CmpPoly = &(*InPolyList)[poly2];

				// We can't compare to ourselves.
				if( CmpPoly == SrcPoly )
					continue;

				for( int32 edge2 = 0 ; edge2 < CmpPoly->Vertices.Num() ; edge2++ )
				{
					FEdge CmpEdge = FEdge( (FVector)CmpPoly->Vertices[edge2], (FVector)CmpPoly->Vertices[ edge2+1 < CmpPoly->Vertices.Num() ? edge2+1 : 0 ] );

					// If both vertices on this edge lie on the same plane as the original edge, create
					// a sphere around the original 2 vertices.  If either of this edges vertices are inside of
					// that sphere, we need to split the original edge by adding a vertex to it's poly.
					if( FMath::Abs( FVector::PointPlaneDist( CmpEdge.Vertex[0], SrcEdge.Vertex[0], SrcEdgePlane ) ) < THRESH_POINT_ON_PLANE
						&& FMath::Abs( FVector::PointPlaneDist( CmpEdge.Vertex[1], SrcEdge.Vertex[0], SrcEdgePlane ) ) < THRESH_POINT_ON_PLANE )
					{
						//
						// Check THIS edge against the SOURCE edge
						//

						FVector Dir = SrcEdge.Vertex[1] - SrcEdge.Vertex[0];
						Dir.Normalize();
						const float Dist = static_cast<float>(FVector::Dist(SrcEdge.Vertex[1], SrcEdge.Vertex[0]));
						FVector Origin = SrcEdge.Vertex[0] + (Dir * (Dist / 2.0f));
						float Radius = Dist / 2.0f;

						for( int32 vtx = 0 ; vtx < 2 ; vtx++ )
							if( FVector::Dist( Origin, CmpEdge.Vertex[vtx] ) && FVector::Dist( Origin, CmpEdge.Vertex[vtx] ) < Radius )
								NewPoly.InsertVertex( edge2+1, CmpEdge.Vertex[vtx] );
					}
				}
			}
		}

		InResult->Add( NewPoly );
	}

}


void FBSPUtils::polyGetOuterEdgeList
(
	TArray<FPoly>* InPolyList,
	TArray<FEdge>* InEdgeList
)
{
	TArray<FPoly> NewPolyList;
	polySplitOverlappingEdges( InPolyList, &NewPolyList );

	TArray<FEdge> TempEdges;

	// Create a list of edges.
	for( int32 poly = 0 ; poly < NewPolyList.Num() ; poly++ )
	{
		FPoly* Poly = &NewPolyList[poly];
		for( int32 vtx = 0 ; vtx < Poly->Vertices.Num() ; vtx++ )
			TempEdges.Emplace( (FVector)Poly->Vertices[vtx], (FVector)Poly->Vertices[ vtx+1 < Poly->Vertices.Num() ? vtx+1 : 0] );
	}

	// Add all the unique edges into the final edge list.
	TArray<FEdge> FinalEdges;

	for( int32 tedge = 0 ; tedge < TempEdges.Num() ; tedge++ )
	{
		FEdge* TestEdge = &TempEdges[tedge];

		int32 EdgeCount = 0;
		for( int32 edge = 0 ; edge < TempEdges.Num() ; edge++ )
		{
			if( TempEdges[edge] == *TestEdge )
				EdgeCount++;
		}

		if( EdgeCount == 1 )
			FinalEdges.Add( *TestEdge );
	}

	// Reorder all the edges so that they line up, end to end.
	InEdgeList->Empty();
	if( !FinalEdges.Num() ) return;

	InEdgeList->Add( FinalEdges[0] );
	FVector Comp = FinalEdges[0].Vertex[1];
	FinalEdges.RemoveAt(0);

	FEdge DebuG;
	for( int32 x = 0 ; x < FinalEdges.Num() ; x++ )
	{
		DebuG = FinalEdges[x];

		// If the edge is backwards, flip it
		if( FinalEdges[x].Vertex[1] == Comp )
			Exchange( FinalEdges[x].Vertex[0], FinalEdges[x].Vertex[1] );

		if( FinalEdges[x].Vertex[0] == Comp )
		{
			InEdgeList->Add( FinalEdges[x] );
			Comp = FinalEdges[x].Vertex[1];
			FinalEdges.RemoveAt(x);
			x = -1;
		}
	}

}

