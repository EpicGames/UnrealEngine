// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Brush.h"
#include "Engine/Polys.h"

class AVolume;
class UModel;

class FBSPOps
{
public:
	/** Quality level for rebuilding Bsp. */
	enum EBspOptimization
	{
		BSP_Lame,
		BSP_Good,
		BSP_Optimal
	};

	/** Possible positions of a child Bsp node relative to its parent (for BspAddToNode) */
	enum ENodePlace 
	{
		NODE_Back		= 0, // Node is in back of parent              -> Bsp[iParent].iBack.
		NODE_Front		= 1, // Node is in front of parent             -> Bsp[iParent].iFront.
		NODE_Plane		= 2, // Node is coplanar with parent           -> Bsp[iParent].iPlane.
		NODE_Root		= 3, // Node is the Bsp root and has no parent -> Bsp[0].
	};

	BSPUTILS_API static void csgPrepMovingBrush( ABrush* Actor );
	BSPUTILS_API static void csgCopyBrush( ABrush* Dest, ABrush* Src, uint32 PolyFlags, EObjectFlags ResFlags, bool bNeedsPrep, bool bCopyPosRotScale, bool bAllowEmpty = false );
	BSPUTILS_API static ABrush*	csgAddOperation( ABrush* Actor, uint32 PolyFlags, EBrushType BrushType );
	BSPUTILS_API static int32 bspAddVector( UModel* Model, const FVector* V, bool Exact );
	BSPUTILS_API static int32 bspAddPoint( UModel* Model, const FVector* V, bool Exact );
	BSPUTILS_API static void bspBuild( UModel* Model, enum EBspOptimization Opt, int32 Balance, int32 PortalBias, int32 RebuildSimplePolys, int32 iNode );
	BSPUTILS_API static void bspRefresh( UModel* Model, bool NoRemapSurfs );
	BSPUTILS_API static void bspBuildBounds( UModel* Model );
	BSPUTILS_API static void bspValidateBrush( UModel* Brush, bool ForceValidate, bool DoStatusUpdate );
	BSPUTILS_API static void bspUnlinkPolys( UModel* Brush );
	BSPUTILS_API static int32 bspAddNode( UModel* Model, int32 iParent, enum ENodePlace ENodePlace, uint32 NodeFlags, FPoly* EdPoly );

	/**
	 * Rebuild some brush internals
	 */
	BSPUTILS_API static void RebuildBrush(UModel* Brush);

	BSPUTILS_API static FPoly BuildInfiniteFPoly( UModel* Model, int32 iNode );

	/** Called when an AVolume shape is changed*/
	BSPUTILS_API static void HandleVolumeShapeChanged(AVolume& Volume);

	/** Errors encountered in Csg operation. */
	BSPUTILS_API static int32 GErrors;
	BSPUTILS_API static bool GFastRebuild;

protected:
	/**
	 * Rotates the specified brush's vertices.
	 */
	static void RotateBrushVerts(ABrush* Brush, const FRotator& Rotation, bool bClearComponents);

	static void SplitPolyList
	(
		UModel				*Model,
		int32               iParent,
		ENodePlace			NodePlace,
		int32               NumPolys,
		FPoly				**PolyList,
		EBspOptimization	Opt,
		int32				Balance,
		int32				PortalBias,
		int32				RebuildSimplePolys
	);
};

/** @todo: remove when uses of ENodePlane have been replaced with FBSPOps::ENodePlace. */
typedef FBSPOps::ENodePlace ENodePlace;


struct FBspPointsKey
{
	int32 X;
	int32 Y;
	int32 Z;

	FBspPointsKey(int32 InX, int32 InY, int32 InZ)
		: X(InX)
		, Y(InY)
		, Z(InZ)
	{}

	friend FORCEINLINE bool operator == (const FBspPointsKey& A, const FBspPointsKey& B)
	{
		return A.X == B.X && A.Y == B.Y && A.Z == B.Z;
	}

	friend FORCEINLINE uint32 GetTypeHash(const FBspPointsKey& Key)
	{
		return HashCombine(static_cast<uint32>(Key.X), HashCombine(static_cast<uint32>(Key.Y), static_cast<uint32>(Key.Z)));
	}
};

struct FBspIndexedPoint
{
	FBspIndexedPoint(const FVector& InPoint, int32 InIndex)
		: Point(InPoint)
		, Index(InIndex)
	{}

	FVector Point;
	int32 Index;
};


struct FBspPointsGridItem
{
	TArray<FBspIndexedPoint, TInlineAllocator<16>> IndexedPoints;
};


// Represents a sparse granular 3D grid into which points are added for quick (~O(1)) lookup.
// The 3D space is divided into a grid with a given granularity.
// Points are considered to have a given radius (threshold) and are added to the grid cube they fall in, and to up to seven neighbours if they overlap.
class FBspPointsGrid
{
public:
	FBspPointsGrid(float InGranularity, float InThreshold, int32 InitialSize = 0)
		: OneOverGranularity(1.0f / InGranularity)
		, Threshold(InThreshold)
	{
		check(InThreshold / InGranularity <= 0.5f);
		Clear(InitialSize);
	}

	BSPUTILS_API void Clear(int32 InitialSize = 0);

	BSPUTILS_API int32 FindOrAddPoint(const FVector& Point, int32 Index, float Threshold);

	BSPUTILS_API static FBspPointsGrid* GBspPoints;
	BSPUTILS_API static FBspPointsGrid* GBspVectors;

private:
	float OneOverGranularity;
	float Threshold;

	typedef TMap<FBspPointsKey, FBspPointsGridItem> FGridMap;
	FGridMap GridMap;
};
