// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Brush.h"

#define UE_API BSPUTILS_API

class UWorld;
class UModel;
class UMaterialInterface;
class FPoly;


class FBSPUtils
{
public:
	/** Repartition Bsp tree */
	static UE_API void bspRepartition( UWorld* InWorld, int32 iNode );

	/** Convert a Bsp node to an EdPoly.  Returns number of vertices in Bsp node. */
	static UE_API int32 bspNodeToFPoly( UModel* Model, int32 iNode, FPoly* EdPoly );

	/**
	 * Clean up all nodes after a CSG operation.  Resets temporary bit flags and unlinks
	 * empty leaves.  Removes zero-vertex nodes which have nonzero-vertex coplanars.
	 */
	static UE_API void bspCleanup( UModel* Model );

	/** 
	 * Build EdPoly list from a model's Bsp. Not transactional.
	 * @param DestArray helps build bsp FPolys in non-main threads. It also allows to perform this action without GUndo 
	 *	      interfering. Temporary results will be written to DestArray. Defaults to Model->Polys->Element
	 */
	static UE_API void bspBuildFPolys( UModel* Model, bool SurfLinks, int32 iNode, TArray<FPoly>* DestArray = NULL );

	static UE_API void bspMergeCoplanars( UModel* Model, bool RemapLinks, bool MergeDisparateTextures );

	/**
	 * Performs any CSG operation between the brush and the world.
	 */
	static UE_API int32 bspBrushCSG( ABrush* Actor, UModel* Model, UModel* TempModel, UMaterialInterface* SelectedMaterialInstance, uint32 PolyFlags, EBrushType BrushType, ECsgOper CSGOper, bool bBuildBounds, bool bMergePolys, bool bReplaceNULLMaterialRefs, bool bShowProgressBar=true );

	/**
	 * Optimize a level's Bsp, eliminating T-joints where possible, and building side
	 * links.  This does not always do a 100% perfect job, mainly due to imperfect 
	 * levels, however it should never fail or return incorrect results.
	 */
	static UE_API void bspOptGeom( UModel* Model );

	/**
	 * Sets and clears all Bsp node flags.  Affects all nodes, even ones that don't
	 * really exist.
	 */
	static UE_API void polySetAndClearPolyFlags(UModel *Model, uint32 SetBits, uint32 ClearBits,bool SelectedOnly, bool UpdateBrush);

	/**
	 *
	 * Find the Brush EdPoly corresponding to a given Bsp surface.
	 *
	 * @param InModel	Model to get poly from
	 * @param iSurf		surface index
	 * @param Poly		
	 *
	 * returns true if poly not available
	 */
	static UE_API bool polyFindBrush(UModel* InModel, int32 iSurf, FPoly &Poly);

	UE_DEPRECATED(5.1, "polyFindMaster is deprecated; please use polyFindBrush instead")
	static UE_API bool polyFindMaster(UModel* InModel, int32 iSurf, FPoly &Poly);

	/**
	 * Update the brush EdPoly corresponding to a newly-changed
	 * poly to reflect its new properties.
	 *
	 * Doesn't do any transaction tracking.
	 */
	static UE_API void polyUpdateBrush(UModel* Model, int32 iSurf, bool bUpdateTexCoords, bool bOnlyRefreshSurfaceMaterials);
	
	UE_DEPRECATED(5.1, "polyUpdateMaster is deprecated; please use polyUpdateBrush instead")
	static UE_API void polyUpdateMaster(UModel* Model, int32 iSurf, bool bUpdateTexCoords, bool bOnlyRefreshSurfaceMaterials);
	
	/**
	 * Populates a list with all polys that are linked to the specified poly.  The
	 * resulting list includes the original poly.
	 */
	static UE_API void polyGetLinkedPolys(ABrush* InBrush, FPoly* InPoly, TArray<FPoly>* InPolyList);

	/**
	 * Takes a list of polygons and creates a new list of polys which have no overlapping edges.  It splits
	 * edges as necessary to achieve this.
	 */
	static UE_API void polySplitOverlappingEdges( TArray<FPoly>* InPolyList, TArray<FPoly>* InResult );

	/**
	 * Takes a list of polygons and returns a list of the outside edges (edges which are not shared
	 * by other polys in the list).
	 */
	static UE_API void polyGetOuterEdgeList(TArray<FPoly>* InPolyList, TArray<FEdge>* InEdgeList);
};

#undef UE_API
